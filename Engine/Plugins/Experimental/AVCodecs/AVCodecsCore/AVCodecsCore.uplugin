{
	"FileVersion": 3,
	"Version": 0.1,
	"VersionName": "0.1",
	"FriendlyName": "AVCodecs Core",
	"Description": "Core Plugin for various Audio/Video codecs",
	"Category": "Codecs",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AVCodecsCore",
			"Type": "Runtime",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ]
		},
		{
			"Name": "AVCodecsCoreRHI",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"TargetDenyList": [
				"Program"
			],
			"PlatformAllowList": [ "Win64", "Linux", "Mac", "Android", "IOS" ]
		}
	],
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac",
		"Android", 
		"IOS"
	]
}