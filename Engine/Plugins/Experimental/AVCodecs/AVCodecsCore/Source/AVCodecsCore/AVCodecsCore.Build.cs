// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System;

[SupportedPlatforms("Win64", "Linux", "Mac", "Android", "IOS")]
public class AVCodecsCore : ModuleRules
{
    public AVCodecsCore(ReadOnlyTargetRules Target) : base(Target)
    {
		bRequiresPlatformSDK = true;
		
        // Without these two compilation fails on VS2017 with D8049: command line is too long to fit in debug record.
        bLegacyPublicIncludePaths = false;
        DefaultBuildSettings = BuildSettingsVersion.Latest;

        PublicIncludePaths.AddRange(new string[] {
			// ... add public include paths required here ...
		});

        PrivateIncludePaths.AddRange(new string[] {
        });

        PrivateDependencyModuleNames.AddRange(new string[] {
            "CoreUObject",
        });

        PublicDependencyModuleNames.AddRange(new string[] {
            "Core",
			// ... add other public dependencies that you statically link with here ...
		});

        DynamicallyLoadedModuleNames.AddRange(new string[] {
			// ... add any modules that your module loads dynamically here ...
		});

        if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) || Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
        {
            PublicIncludePathModuleNames.Add("Vulkan");
            AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan");

            PublicDefinitions.Add("AVCODECS_USE_VULKAN=1");
        }
        else
        {
            PublicDefinitions.Add("AVCODECS_USE_VULKAN=0");
        }

        if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
        {
            AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11", "DX12");

            PublicDelayLoadDLLs.Add("mfplat.dll");
            PublicDelayLoadDLLs.Add("mfuuid.dll");
            PublicDelayLoadDLLs.Add("Mfreadwrite.dll");

            PublicDefinitions.Add("AVCODECS_USE_D3D=1");
        }
        else
        {
            PublicDefinitions.Add("AVCODECS_USE_D3D=0");
        }

        if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
        {
            PublicFrameworks.AddRange(new string[]{
                "AVFoundation",
                "VideoToolbox"
            });

            PublicDependencyModuleNames.AddRange(new string[] {
                "MetalRHI",
            });

            PublicWeakFrameworks.Add("Metal");
            AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");

            PublicDefinitions.Add("AVCODECS_USE_METAL=1");
        }
        else
        {
            PublicDefinitions.Add("AVCODECS_USE_METAL=0");
        }

        PublicDefinitions.Add("DEBUG_DUMP_TO_DISK=0");
    }
}
