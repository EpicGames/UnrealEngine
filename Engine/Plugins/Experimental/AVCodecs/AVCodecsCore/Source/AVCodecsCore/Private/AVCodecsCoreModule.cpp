// Copyright Epic Games, Inc. All Rights Reserved.

#include "AVDevice.h"

#include "Modules/ModuleManager.h"

#include "Video/Resources/VideoResourceCPU.h"

class FAVCodecCoreModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		FAVDevice::GetSoftwareDevice()->SetContext<FVideoContextCPU>(
			MakeShared<FVideoContextCPU>());
	}
};

IMPLEMENT_MODULE(FAVCodecCoreModule, AVCodecsCore);
