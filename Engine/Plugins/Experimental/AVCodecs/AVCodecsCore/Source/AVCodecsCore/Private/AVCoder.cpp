// Copyright Epic Games, Inc. All Rights Reserved.

#include "AVCoder.h"

TMap<FTypeID, TMap<FTypeID, TMap<FTypeID, TSharedPtr<void>>>> IAVCoder::Factories = TMap<FTypeID, TMap<FTypeID, TMap<FTypeID, TSharedPtr<void>>>>();
