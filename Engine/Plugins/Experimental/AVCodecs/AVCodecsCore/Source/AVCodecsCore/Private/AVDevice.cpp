// Copyright Epic Games, Inc. All Rights Reserved.

#include "AVDevice.h"

TSharedRef<FAVDevice>& FAVDevice::GetHardwareDevice(int32 Index)
{
	static TArray<TSharedRef<FAVDevice>> Devices;

	if (Devices.Num() <= Index)
	{
		Devices.EmplaceAt(Index, MakeShared<FAVDevice>());
	}
	
	return Devices[Index];
}

TSharedRef<FAVDevice>& FAVDevice::GetSoftwareDevice()
{
	static TSharedRef<FAVDevice> Device = MakeShared<FAVDevice>();
	return Device;
}
