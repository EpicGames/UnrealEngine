// Copyright Epic Games, Inc. All Rights Reserved.

#include "AVResource.h"

FAVResource::FAVResource(TSharedRef<FAVDevice> const& Device, FAVLayout const& Layout)
	: Device(Device)
	, Layout(Layout)
{
}

FAVResult FAVResource::Validate() const
{
	return EAVResult::Success;
}

void FAVResource::Lock()
{
	Mutex.Lock();
}

FScopeLock FAVResource::LockScope()
{
	return FScopeLock(&Mutex);
}

void FAVResource::Unlock()
{
	Mutex.Unlock();
}
