// Copyright Epic Games, Inc. All Rights Reserved.

#include "AVResult.h"
#include "Async/Async.h"

DEFINE_LOG_CATEGORY(LogAVCodecs);

FString ToString(EAVResult Result)
{
	static auto constexpr Prettify = [](FString RawValue) -> FString
	{
		TStringBuilder<256> ResultString;
		ResultString.Append(RawValue);

		for (int i = 1; i < ResultString.Len(); ++i)
		{
			if (FChar::IsUpper(ResultString.GetData()[i]))
			{
				ResultString.InsertAt(i, TEXT(" "));

				++i;
			}
		}
		
		return FString(ResultString);
	};

// Macro to stringify enum values
#define CASE(Result) \
	case EAVResult::Result: return Prettify(TEXT(#Result));
	
	switch (Result)
	{
	CASE(Unknown);
	
	CASE(Fatal);
	CASE(FatalUnsupported);
	
	CASE(Error);
	CASE(ErrorUnsupported);
	CASE(ErrorInvalidState);
	CASE(ErrorCreating);
	CASE(ErrorDestroying);
	CASE(ErrorResolving);
	CASE(ErrorMapping);
	CASE(ErrorUnmapping);
	CASE(ErrorLocking);
	CASE(ErrorUnlocking);
	
	CASE(Warning);
	CASE(WarningInvalidState);
	
	CASE(Pending);
	CASE(PendingInput);
	CASE(PendingOutput);
	
	CASE(Success);
		
	default:
		return FString::Printf(TEXT("%d"), EnumToUnderlyingType(Result));
	}

#undef CASE
}

FAVResult& FAVResult::Handle()
{
	bHandled = true;

	return *this;
}

AVCODECSCORE_API FString FAVResult::ToString() const
{
	TStringBuilder<256> ResultString;
	ResultString.Append(Message);

	if (!Vendor.IsEmpty())
	{
		if (ResultString.Len() > 0)
		{
			ResultString.Append(TEXT(" "));
		}

		if (VendorValue != EMPTY_VENDOR_VALUE)
		{
			ResultString.Appendf(TEXT("[%s %d]"), *Vendor, VendorValue);
		}
		else
		{
			ResultString.Appendf(TEXT("[%s]"), *Vendor);
		}
	}

	if (ResultString.Len() > 0)
	{
		ResultString.InsertAt(0, TEXT(": "));
	}

	ResultString.InsertAt(0, ::ToString(Value));

	return FString(ResultString);
}

void FAVResult::Log() const
{
	if (IsInGameThread())
	{
		if (IsFatal())
		{
			UE_LOGF(LogAVCodecs, Fatal, "%ls", *ToString());
		}
		else if (IsError())
		{
			UE_LOGF(LogAVCodecs, Error, "%ls", *ToString());
		}
		else if (IsWarning())
		{
			UE_LOGF(LogAVCodecs, Warning, "%ls", *ToString());
		}
		else if (IsPending())
		{
			UE_LOGF(LogAVCodecs, Verbose, "%ls", *ToString());
		}
		else if (IsSuccess())
		{
			UE_LOGF(LogAVCodecs, Verbose, "%ls", *ToString());
		}
		else
		{
			UE_LOGF(LogAVCodecs, Log, "%ls", *ToString());
		}
	}
	else
	{
		AsyncTask(ENamedThreads::GameThread, [Self = *this]()
		{
			Self.Log();
		});
	}
}
