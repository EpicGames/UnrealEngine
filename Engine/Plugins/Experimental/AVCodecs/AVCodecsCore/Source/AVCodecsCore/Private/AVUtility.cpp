// Copyright Epic Games, Inc. All Rights Reserved.

#include "AVUtility.h"

TTypeMap<FAPI> FAPI::Singletons;
