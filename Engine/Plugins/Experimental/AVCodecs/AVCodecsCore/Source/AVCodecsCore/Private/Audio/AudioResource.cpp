// Copyright Epic Games, Inc. All Rights Reserved.

#include "Audio/AudioResource.h"

FAudioResource::FAudioResource(TSharedRef<FAVDevice> const& Device, FAVLayout const& Layout, FAudioDescriptor const& Descriptor)
	: FAVResource(Device, Layout)
	, Descriptor(Descriptor)
{
}
