// Copyright Epic Games, Inc. All Rights Reserved.

#include "Audio/Encoders/Configs/AudioEncoderConfigAAC.h"

#include "AVUtility.h"

REGISTER_TYPEID(FAudioEncoderConfigAAC);
