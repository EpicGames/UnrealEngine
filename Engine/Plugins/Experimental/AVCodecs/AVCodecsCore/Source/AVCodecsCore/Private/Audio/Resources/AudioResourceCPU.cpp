// Copyright Epic Games, Inc. All Rights Reserved.

#include "Audio/Resources/AudioResourceCPU.h"

REGISTER_TYPEID(FAudioContextCPU);
REGISTER_TYPEID(FAudioResourceCPU);

FAudioContextCPU::FAudioContextCPU()
{
}

FAudioResourceCPU::FAudioResourceCPU(TSharedRef<FAVDevice> const& Device, TSharedPtr<float> const& Raw, FAVLayout const& Layout, FAudioDescriptor const& Descriptor)
	: TAudioResource(Device, Layout, Descriptor)
	, Raw(Raw)
{
}

FAVResult FAudioResourceCPU::Validate() const
{
	if (!Raw.IsValid())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Raw resource is invalid"), TEXT("CPU"));
	}

	return EAVResult::Success;
}
