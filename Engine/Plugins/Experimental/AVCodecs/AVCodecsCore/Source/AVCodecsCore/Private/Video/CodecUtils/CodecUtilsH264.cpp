// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/CodecUtils/CodecUtilsH264.h"

#include "Containers/Array.h"
#include "AVResult.h"
#include "Video/VideoPacket.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CodecUtilsH264)

FH264ProfileDefinition GH264ProfileDefinitions[static_cast<uint8>(EH264Profile::MAX)] = {
	{ EH264Profile::Auto, UE::AVCodecCore::H264::EH264ProfileIDC::Auto, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("Auto") },
	{ EH264Profile::CALVLC444Intra, UE::AVCodecCore::H264::EH264ProfileIDC::CALVLC444Intra, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("CALVLC444Intra") },
	{ EH264Profile::Baseline, UE::AVCodecCore::H264::EH264ProfileIDC::Baseline, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("Baseline") },
	{ EH264Profile::ConstrainedBaseline, UE::AVCodecCore::H264::EH264ProfileIDC::Baseline, UE::AVCodecCore::H264::EH264ConstraintFlag::Set1, TEXT("ConstrainedBaseline") },
	{ EH264Profile::Main, UE::AVCodecCore::H264::EH264ProfileIDC::Main, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("Main") },
	{ EH264Profile::ScalableBaseline, UE::AVCodecCore::H264::EH264ProfileIDC::ScalableBaseline, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("ScalableBaseline") },
	{ EH264Profile::ScalableConstrainedBaseline, UE::AVCodecCore::H264::EH264ProfileIDC::ScalableBaseline, UE::AVCodecCore::H264::EH264ConstraintFlag::Set5, TEXT("ScalableConstrainedBaseline") },
	{ EH264Profile::ScalableHigh, UE::AVCodecCore::H264::EH264ProfileIDC::ScalableHigh, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("ScalableHigh") },
	{ EH264Profile::ScalableConstrainedHigh, UE::AVCodecCore::H264::EH264ProfileIDC::ScalableHigh, UE::AVCodecCore::H264::EH264ConstraintFlag::Set5, TEXT("ScalableConstrainedHigh") },
	{ EH264Profile::ScalableHighIntra, UE::AVCodecCore::H264::EH264ProfileIDC::ScalableHigh, UE::AVCodecCore::H264::EH264ConstraintFlag::Set3, TEXT("ScalableHighIntra") },
	{ EH264Profile::Extended, UE::AVCodecCore::H264::EH264ProfileIDC::Extended, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("Extended") },
	{ EH264Profile::High, UE::AVCodecCore::H264::EH264ProfileIDC::High, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("High") },
	{ EH264Profile::ProgressiveHigh, UE::AVCodecCore::H264::EH264ProfileIDC::High, UE::AVCodecCore::H264::EH264ConstraintFlag::Set4, TEXT("ProgressiveHigh") },
	{ EH264Profile::ConstrainedHigh, UE::AVCodecCore::H264::EH264ProfileIDC::High, UE::AVCodecCore::H264::EH264ConstraintFlag::Set4& UE::AVCodecCore::H264::EH264ConstraintFlag::Set5, TEXT("ConstrainedHigh") },
	{ EH264Profile::High10, UE::AVCodecCore::H264::EH264ProfileIDC::High10, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("High10") },
	{ EH264Profile::High10Intra, UE::AVCodecCore::H264::EH264ProfileIDC::High10, UE::AVCodecCore::H264::EH264ConstraintFlag::Set3, TEXT("High10Intra") },
	{ EH264Profile::MultiviewHigh, UE::AVCodecCore::H264::EH264ProfileIDC::MultiviewHigh, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("MultiviewHigh") },
	{ EH264Profile::High422, UE::AVCodecCore::H264::EH264ProfileIDC::High422, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("High422") },
	{ EH264Profile::High422Intra, UE::AVCodecCore::H264::EH264ProfileIDC::High422, UE::AVCodecCore::H264::EH264ConstraintFlag::Set3, TEXT("High422Intra") },
	{ EH264Profile::StereoHigh, UE::AVCodecCore::H264::EH264ProfileIDC::StereoHigh, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("StereoHigh") },
	{ EH264Profile::MultiviewDepthHigh, UE::AVCodecCore::H264::EH264ProfileIDC::MultiviewDepthHigh, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("MultiviewDepthHigh") },
	{ EH264Profile::High444, UE::AVCodecCore::H264::EH264ProfileIDC::High444, UE::AVCodecCore::H264::EH264ConstraintFlag::None, TEXT("High444") },
	{ EH264Profile::High444Intra, UE::AVCodecCore::H264::EH264ProfileIDC::High444, UE::AVCodecCore::H264::EH264ConstraintFlag::Set3, TEXT("High444Intra") }
};

namespace UE::AVCodecCore::H264
{
	FAVResult FindNALUs(FVideoPacket const& InPacket, TArray<FNaluH264>& FoundNalus)
	{
		FBitstreamReader Reader(InPacket.DataPtr.Get(), InPacket.DataSize);
		if (Reader.NumBytesRemaining() < 3)
		{
			return FAVResult(EAVResult::Warning, TEXT("Bitstream not long enough to hold a NALU"), TEXT("BitstreamParserH264"));
		}

		TArrayView64<uint8> const Data = InPacket.GetData();
		
		// Skip over stream in intervals of 3 until Data[i + 2] is either 0 or 1
		for (int64 i = 0; i < Data.Num() - 2;)
		{
			if (Data[i + 2] > 1)
			{
				i += 3;
			}
			else if (Data[i + 2] == 1)
			{
				if (Data[i + 1] == 0 && Data[i] == 0)
				{
					// Found start sequence of NalU but we don't know if it has a 3 or 4 byte start code so we check.
					FNaluH264 NalInfo = { (uint64)i, 0, 3, 0, ENaluType::Unspecified, nullptr };
					if (NalInfo.Start > 0 && Data[NalInfo.Start - 1] == 0)
					{
						++NalInfo.StartCodeSize;
						--NalInfo.Start;
					}

					FBitstreamReader Bitstream(&Data[NalInfo.Start + NalInfo.StartCodeSize], 1);
					verifyf(Bitstream.ReadBits(1) == 0, TEXT("Forbidden Zero bit not Zero in NAL Header"));

					NalInfo.RefIdc = Bitstream.ReadBits(2);
					NalInfo.Type = (ENaluType)Bitstream.ReadBits(5);

					NalInfo.Data = &Data[NalInfo.Start + NalInfo.StartCodeSize + 1];

					// Update length of previous entry.
					if (FoundNalus.Num() > 0)
					{
						FoundNalus.Last().Size = NalInfo.Start - (FoundNalus.Last().Start + FoundNalus.Last().StartCodeSize);
					}

					FoundNalus.Add(NalInfo);
				}

				i += 3;
			}
			else
			{
				++i;
			}
		}

		if (FoundNalus.Num() == 0)
		{
			return FAVResult(EAVResult::PendingInput, TEXT("no NALUs found in BitDataStream"), TEXT("BitstreamParserH264"));
		}

		// Last Nal size is the remaining size of the bitstream minus a trailing zero byte
		FoundNalus.Last().Size = Data.Num() - (FoundNalus.Last().Start + FoundNalus.Last().StartCodeSize);

	#if !IS_PROGRAM

		FAVResult::Log(EAVResult::Success, FString::Printf(TEXT("FAVExtension::TransformConfig found %d NALUs in bitdatastream"), FoundNalus.Num()));

		for (const FNaluH264& NalUInfo : FoundNalus)
		{
			FAVResult::Log(EAVResult::Success, FString::Printf(TEXT("Found NALU at %llu size %llu with type %u"), NalUInfo.Start, NalUInfo.Size, (uint8)NalUInfo.Type));
		}

	#endif //!IS_PROGRAM

		return EAVResult::Success;
	}

	FAVResult ParseSEI(FBitstreamReader& Bitstream, FNaluH264 const& InNaluInfo, SEI_t& OutSEI)
	{
		//unimplemented();
		return FAVResult(EAVResult::Success, TEXT("SEI Unimplemented"));
	}

	void ParseScalingList(FBitstreamReader& Bitstream, const uint8& chroma_format_idc,  FNalu::U<1> scaling_list_present_flag[12], uint8 ScalingList4x4[6][16], uint8 ScalingList8x8[6][64])
	{
		auto scaling_list = [&Bitstream](uint8* scalingList, int32 sizeOfScalingList, bool& useDefaultScalingMatrixFlag) -> void
		{
			int32 lastScale = 8;
			int32 nextScale = 8;
			for (int32 j = 0; j < sizeOfScalingList; ++j)
			{
				if (nextScale)
				{
					FNalu::SE delta_scale;
					Bitstream.Read(delta_scale);
					
					nextScale = (lastScale + delta_scale + 256) % 256;
					useDefaultScalingMatrixFlag = (j == 0 && nextScale == 0);
				}
				scalingList[j] = (nextScale == 0) ? lastScale : nextScale;
				lastScale = scalingList[j];
			}
		};

		bool useDefaultScalingMatrixFlag = false;
		for (int32 i = 0, iMax = (chroma_format_idc != 3) ? 8 : 12; i < iMax; ++i)
		{
			Bitstream.Read(scaling_list_present_flag[i]);
			if (scaling_list_present_flag[i])
			{
				if (i < 6)
				{
					scaling_list(ScalingList4x4[i], 16, useDefaultScalingMatrixFlag);
					if (useDefaultScalingMatrixFlag)
					{
						if (i < 3)
						{
							FMemory::Memcpy(ScalingList4x4[i], Default_4x4_Intra, 16);
						}
						else
						{
							FMemory::Memcpy(ScalingList4x4[i], Default_4x4_Inter, 16);									
						}
					}
				}
				else
				{
					scaling_list(ScalingList8x8[i - 6], 64, useDefaultScalingMatrixFlag);
					if (useDefaultScalingMatrixFlag)
					{
						if (i % 2)
						{
							FMemory::Memcpy(ScalingList8x8[i - 6], Default_8x8_Intra, 64);
						}
						else
						{
							FMemory::Memcpy(ScalingList8x8[i - 6], Default_8x8_Inter, 64);									
						}
					}
				}
			}
		}
	}

	FAVResult ParseSPS(FBitstreamReader& Bitstream, FNaluH264 const& InNaluInfo, TMap<uint32, SPS_t>& OutMapSPS)
	{
		// Be sure to maintain parity with SPS struct values, or we may miss bits
		FNalu::U<8, EH264ProfileIDC> profile_idc;
        Bitstream.Read(profile_idc); // u(8)

		FNalu::U<8, EH264ConstraintFlag> constraint_flags;
        Bitstream.Read(constraint_flags); // u(8)

        FNalu::U<8> level_idc;
		Bitstream.Read(level_idc); // u(8)
		
		FNalu::UE seq_parameter_set_id;
		Bitstream.Read(seq_parameter_set_id); // ue(v)

		// Find SPS at current ID or set it to the map
		SPS_t& OutSPS = OutMapSPS.FindOrAdd(seq_parameter_set_id);

        OutSPS.profile_idc = profile_idc;
		OutSPS.constraint_flags = constraint_flags;
        OutSPS.level_idc = level_idc;
        OutSPS.seq_parameter_set_id = seq_parameter_set_id;

		if (OutSPS.profile_idc == EH264ProfileIDC::High ||
			OutSPS.profile_idc == EH264ProfileIDC::High10 ||
			OutSPS.profile_idc == EH264ProfileIDC::High422 ||
			OutSPS.profile_idc == EH264ProfileIDC::High444 ||
			OutSPS.profile_idc == EH264ProfileIDC::CALVLC444Intra ||
			OutSPS.profile_idc == EH264ProfileIDC::ScalableBaseline ||
			OutSPS.profile_idc == EH264ProfileIDC::ScalableHigh ||
			OutSPS.profile_idc == EH264ProfileIDC::MultiviewHigh ||
            OutSPS.profile_idc == EH264ProfileIDC::StereoHigh ||
			OutSPS.profile_idc == EH264ProfileIDC::MultiviewDepthHigh ||
            OutSPS.profile_idc == EH264ProfileIDC::MultiresolutionFrameCompatibleHigh ||
            OutSPS.profile_idc == EH264ProfileIDC::EnhancedMultiviewDepthHigh)
		{
			Bitstream.Read(OutSPS.chroma_format_idc); // ue(v)

			if (OutSPS.chroma_format_idc == 3)
			{
				Bitstream.Read(OutSPS.separate_colour_plane_flag); // u(1)
			}

			Bitstream.Read(
				OutSPS.bit_depth_luma_minus8,                // ue(v)
				OutSPS.bit_depth_chroma_minus8,              // ue(8)
				OutSPS.qpprime_y_zero_transform_bypass_flag, // u(1)
				OutSPS.seq_scaling_matrix_present_flag);     // u(1)

			if (OutSPS.seq_scaling_matrix_present_flag)
			{
				ParseScalingList(Bitstream, OutSPS.chroma_format_idc, OutSPS.seq_scaling_list_present_flag, OutSPS.ScalingList4x4, OutSPS.ScalingList8x8);
			}
		}

        const uint32_t MaxLog2Minus4 = 12;
		Bitstream.Read(OutSPS.log2_max_frame_num_minus4); // ue(v)
        if(OutSPS.log2_max_frame_num_minus4 > MaxLog2Minus4)
        {
			return FAVResult(EAVResult::Error, TEXT("log2_max_frame_num_minus4 > MaxLog2Minus4"), TEXT("H264"));
        }

		Bitstream.Read(OutSPS.pic_order_cnt_type); // ue(v)
		if (OutSPS.pic_order_cnt_type == 0)
		{
			Bitstream.Read(OutSPS.log2_max_pic_order_cnt_lsb_minus4); // ue(v)
            if(OutSPS.log2_max_pic_order_cnt_lsb_minus4 > MaxLog2Minus4)
            {
                return FAVResult(EAVResult::Error, TEXT("log2_max_pic_order_cnt_lsb_minus4 > MaxLog2Minus4"), TEXT("H264"));
            }
		}
		else if (OutSPS.pic_order_cnt_type == 1)
		{
			Bitstream.Read(
				OutSPS.delta_pic_order_always_zero_flag,       // u(1)
				OutSPS.offset_for_non_ref_pic,                 // se(v)
				OutSPS.offset_for_top_to_bottom_field,         // se(v)
				OutSPS.num_ref_frames_in_pic_order_cnt_cycle); // ue(v)
			
			for (uint32 i = 0; i < OutSPS.num_ref_frames_in_pic_order_cnt_cycle; ++i)
			{
				Bitstream.Read(OutSPS.offset_for_ref_frame[i]); // se(v)
			}
		}

		Bitstream.Read(
			OutSPS.max_num_ref_frames,                   // ue(v)
			OutSPS.gaps_in_frame_num_value_allowed_flag, // u(1)
			OutSPS.pic_width_in_mbs_minus1,              // ue(v)
			OutSPS.pic_height_in_map_units_minus1);      // ue(v)

		Bitstream.Read(OutSPS.frame_mbs_only_flag); // u(1)
		if (!OutSPS.frame_mbs_only_flag)
		{
			Bitstream.Read(OutSPS.mb_adaptive_frame_field_flag); // u(1)
		}

		Bitstream.Read(OutSPS.direct_8x8_inference_flag); // u(1)

		Bitstream.Read(OutSPS.frame_cropping_flag); // u(1)
		if (OutSPS.frame_cropping_flag)
		{
			Bitstream.Read(
				OutSPS.frame_crop_left_offset,    // ue(v)
				OutSPS.frame_crop_right_offset,   // ue(v)
				OutSPS.frame_crop_top_offset,     // ue(v)
				OutSPS.frame_crop_bottom_offset); // ue(v)
		}

		Bitstream.Read(OutSPS.vui_parameters_present_flag); // u(1)

        // TODO (william.belcher): Parsing of VUI info

		return EAVResult::Success;
	}

	FAVResult ParsePPS(FBitstreamReader& Bitstream, FNaluH264 const& InNaluInfo, TMap<uint32, SPS_t> const& InMapSPS, TMap<uint32, PPS_t>& OutMapPPS)
	{	
		FNalu::UE pic_parameter_set_id;
		Bitstream.Read(pic_parameter_set_id); // ue(v)

		PPS_t& OutPPS = OutMapPPS.FindOrAdd(pic_parameter_set_id);
		OutPPS.pic_parameter_set_id = pic_parameter_set_id;
		
		Bitstream.Read(
			OutPPS.seq_parameter_set_id,                         // ue(v)
			OutPPS.entropy_coding_mode_flag,                     // u(1)
			OutPPS.bottom_field_pic_order_in_frame_present_flag, // u(1)
			OutPPS.num_slice_groups_minus1);                     // ue(v)
		
		if( OutPPS.num_slice_groups_minus1 > 0 )
		{
			Bitstream.Read(OutPPS.slice_group_map_type); // ue(v)
			if( OutPPS.slice_group_map_type == 0 )
			{
				OutPPS.run_length_minus1.AddUninitialized(OutPPS.num_slice_groups_minus1 + 1);
				for (uint32 iGroup = 0; iGroup <= OutPPS.num_slice_groups_minus1; iGroup++ )
				{
					Bitstream.Read(OutPPS.run_length_minus1[ iGroup ]); // ue(v)
				}
			}
            else if( OutPPS.slice_group_map_type == 1 )
            {
                // TODO (wiliam.belcher): Implement support for dispersed slice group map type.
                // See 8.2.2.2 Specification for dispersed slice group map type.
            }
			else if( OutPPS.slice_group_map_type == 2 )
			{
				OutPPS.top_left.AddUninitialized(OutPPS.num_slice_groups_minus1 + 1);
				OutPPS.bottom_right.AddUninitialized(OutPPS.num_slice_groups_minus1 + 1);
				for (uint32 iGroup = 0; iGroup < OutPPS.num_slice_groups_minus1; iGroup++ )
					{
						Bitstream.Read(
							OutPPS.top_left[ iGroup ],      // ue(v)
							OutPPS.bottom_right[ iGroup ]); // ue(v)
					}
			}
			else if( OutPPS.slice_group_map_type == 3 ||
					 OutPPS.slice_group_map_type == 4 ||
					 OutPPS.slice_group_map_type == 5 )
			{
				Bitstream.Read(
					OutPPS.slice_group_change_direction_flag, // u(1)
					OutPPS.slice_group_change_rate_minus1);   // ue(v)
			}
			else if( OutPPS.slice_group_map_type == 6 )
			{
				Bitstream.Read(OutPPS.pic_size_in_map_units_minus1); // ue(v)
				unsigned long numBits = 0;
			#if defined(_MSC_VER)
					_BitScanForward(&numBits, OutPPS.pic_size_in_map_units_minus1 + 1);
			#else
					numBits = __builtin_ctz(OutPPS.pic_size_in_map_units_minus1 + 1);
			#endif
				for( uint32 i = 0; i <= OutPPS.pic_size_in_map_units_minus1; i++ )
				{
					Bitstream.ReadBits(OutPPS.slice_group_id[i], numBits); // u(numBits)
				}
			}
		}
		
		Bitstream.Read(OutPPS.num_ref_idx_l0_default_active_minus1, // ue(v)
			OutPPS.num_ref_idx_l1_default_active_minus1,            // ue(v)
			OutPPS.weighted_pred_flag,                              // u(1)
			OutPPS.weighted_bipred_idc,                             // u(2)
			OutPPS.pic_init_qp_minus26,                             // se(v)
			OutPPS.pic_init_qs_minus26,                             // se(v)
			OutPPS.chroma_qp_index_offset,                          // se(v)
			OutPPS.deblocking_filter_control_present_flag,          // u(1)
			OutPPS.constrained_intra_pred_flag,                     // u(1)
			OutPPS.redundant_pic_cnt_present_flag);                 // u(1)

		auto more_rbsp_data = [&Bitstream]() -> bool
		{
			if(Bitstream.NumBitsRemaining() > 0)
			{
				return true;
			}

			return false;
		};

        // TODO (william.belcher): Parsing of additional rbsp data
		
		return EAVResult::Success;
	}

	FAVResult ParseSliceHeader(FBitstreamReader& Bitstream, FNaluH264 const& InNaluInfo, TMap<uint32, SPS_t> const& InMapSPS, TMap<uint32, PPS_t> const& InMapPPS, Slice_t& OutSlice)
	{
        Bitstream.Read(OutSlice.first_mb_in_slice,     // ue(v)
                       OutSlice.slice_type,            // ue(v)
                       OutSlice.pic_parameter_set_id); // ue(v)
        
		const PPS_t& CurrentPPS = InMapPPS[OutSlice.pic_parameter_set_id];
		const SPS_t& CurrentSPS = InMapSPS[CurrentPPS.seq_parameter_set_id];

		if (CurrentSPS.separate_colour_plane_flag)
		{
			Bitstream.Read(OutSlice.colour_plane_id); // u(2)
		}

		Bitstream.ReadBits(OutSlice.frame_num, CurrentSPS.log2_max_frame_num_minus4 + 4); // u(v)

		if (!CurrentSPS.frame_mbs_only_flag)
		{
			Bitstream.Read(OutSlice.field_pic_flag); // u(1)
			if (OutSlice.field_pic_flag)
			{
				Bitstream.Read(OutSlice.bottom_field_flag); // u(1)
			}
		}

		if (InNaluInfo.Type == ENaluType::SliceIdrPicture)
		{
			Bitstream.Read(OutSlice.idr_pic_id); // ue(v)
		}

		if (CurrentSPS.pic_order_cnt_type == 0)
		{
			Bitstream.ReadBits(OutSlice.pic_order_cnt_lsb, CurrentSPS.log2_max_pic_order_cnt_lsb_minus4 + 4); // u(v)

			if (CurrentPPS.bottom_field_pic_order_in_frame_present_flag && !OutSlice.field_pic_flag)
			{
				Bitstream.Read(OutSlice.delta_pic_order_cnt_bottom); // se(v)
			}
		}

		if (CurrentSPS.pic_order_cnt_type == 1 && !CurrentSPS.delta_pic_order_always_zero_flag)
		{
			Bitstream.Read(OutSlice.delta_pic_order_cnt[0]); // se(v)

			if (CurrentPPS.bottom_field_pic_order_in_frame_present_flag && !OutSlice.field_pic_flag)
			{
				Bitstream.Read(OutSlice.delta_pic_order_cnt[1]); // se(v)
			}
		}

		if (CurrentPPS.redundant_pic_cnt_present_flag)
		{
			Bitstream.Read(OutSlice.redundant_pic_cnt); // ue(v)
		}

		const bool IsP = OutSlice.slice_type == 0 || OutSlice.slice_type == 5;
		const bool IsB = OutSlice.slice_type == 1 || OutSlice.slice_type == 6;
		const bool IsI = OutSlice.slice_type == 2 || OutSlice.slice_type == 7;
		const bool IsSP = OutSlice.slice_type == 3 || OutSlice.slice_type == 8;
		const bool IsSI = OutSlice.slice_type == 4 || OutSlice.slice_type == 9;
		
		if (IsB)
		{
			Bitstream.Read(OutSlice.direct_spatial_mv_pred_flag); // u(1)
		}

		if ( IsP || IsSP || IsB )
		{
			Bitstream.Read(OutSlice.num_ref_idx_active_override_flag); // u(1)

			if (OutSlice.num_ref_idx_active_override_flag)
			{
				Bitstream.Read(OutSlice.num_ref_idx_l0_active_minus1); // ue(v)

				if ( IsB )
				{
					Bitstream.Read(OutSlice.num_ref_idx_l1_active_minus1); // ue(v)
				}
			}
		}

		// Slice extensions
		if (InNaluInfo.Type == ENaluType::SliceExtension || InNaluInfo.Type == ENaluType::SliceExtensionForDepthView)
		{
			// TODO ref_pic_list_mvc_modification()
		}
		else
		{
			if ( OutSlice.slice_type % 5 != 2 &&  OutSlice.slice_type % 5 != 4 )
			{
				Bitstream.Read(OutSlice.ref_pic_list_modification_flag_l0); // u(1)
				if (OutSlice.ref_pic_list_modification_flag_l0)
				{
					FNalu::UE modification_of_pic_nums_idc;
					do
					{
						Bitstream.Read(modification_of_pic_nums_idc); // ue(v)
						if (modification_of_pic_nums_idc == 0 || modification_of_pic_nums_idc == 1)
						{
							FNalu::UE pic_num;
							Bitstream.Read(pic_num); // ue(v)
							
                            // TODO (william.belcher)
							// OutSlice.RefPicList0.Add({false, pic_num});	
						}
						else if (modification_of_pic_nums_idc == 2)
						{
							FNalu::UE pic_num;
							Bitstream.Read(pic_num); // ue(v)
							
                            // TODO (william.belcher)
							// OutSlice.RefPicList0.Add({true, pic_num});							
						}
					}
					while (modification_of_pic_nums_idc != 3);
				}
			}

			if (OutSlice.slice_type % 5 == 1)
			{
				Bitstream.Read(OutSlice.ref_pic_list_modification_flag_l1); // u(1)
				if (OutSlice.ref_pic_list_modification_flag_l1)
				{
					FNalu::UE modification_of_pic_nums_idc;
					do
					{
						Bitstream.Read(modification_of_pic_nums_idc); // ue(v)
						if (modification_of_pic_nums_idc == 0 || modification_of_pic_nums_idc == 1)
						{
							FNalu::UE pic_num;
							Bitstream.Read(pic_num); // ue(v)
							
                            // TODO (william.belcher)
							// OutSlice.RefPicList1.Add({false, pic_num});	
						}
						else if (modification_of_pic_nums_idc == 2)
						{
							FNalu::UE pic_num;
							Bitstream.Read(pic_num); // ue(v)
							
                            // TODO (william.belcher)
							// OutSlice.RefPicList1.Add({true, pic_num});							
						}
					}
					while (modification_of_pic_nums_idc != 3);
				}
			}
		}

		if ( (CurrentPPS.weighted_pred_flag && ( IsP || IsSP )) || (CurrentPPS.weighted_bipred_idc == 1 && IsB) )
		{
			// TODO pred_weight_table();
		}

		if (InNaluInfo.RefIdc != 0)
		{
			// TODO dec_ref_pic_marking()
            if(InNaluInfo.Type == ENaluType::SliceIdrPicture)
            {
                Bitstream.Read(OutSlice.no_output_of_prior_pic_flag, // u(1)
                               OutSlice.long_term_reference_flag);   // u(1)
            }
            else
            {
                Bitstream.Read(OutSlice.adaptive_ref_pic_marking_mode_flag); // u(1)
                if(OutSlice.adaptive_ref_pic_marking_mode_flag)
                {
                    FNalu::UE memory_management_control_operation;
                    do
                    {
                        Bitstream.Read(memory_management_control_operation); // ue(v)
                        if(memory_management_control_operation == 1 || memory_management_control_operation == 3)
                        {
                            Bitstream.Read(OutSlice.difference_of_pic_nums_minus1); // ue(v)
                        }
                        if(memory_management_control_operation == 2)
                        {
                            Bitstream.Read(OutSlice.long_term_pic_num); // ue(v)
                        }
                        if (memory_management_control_operation == 3 || memory_management_control_operation == 6) 
                        {
                            Bitstream.Read(OutSlice.long_term_frame_idx); // ue(v)
                        }
                        if (memory_management_control_operation == 4) 
                        {
                            Bitstream.Read(OutSlice.max_long_term_frame_idx_plus1); // ue(v)
                        }
                    } 
                    while(memory_management_control_operation != 0);
                }
            }
		}

		if (CurrentPPS.entropy_coding_mode_flag && !IsI && !IsSI)
		{
			Bitstream.Read(OutSlice.cabac_init_idc); // ue(v)
		}

		Bitstream.Read(OutSlice.slice_qp_delta); // se(v)

		if ( IsSP || IsSI )
		{
			if (IsSP)
			{
				Bitstream.Read(OutSlice.sp_for_switch_flag); // u(1)
			}

			Bitstream.Read(OutSlice.slice_qs_delta); // se(v)
		}

		if (CurrentPPS.deblocking_filter_control_present_flag)
		{
			Bitstream.Read(OutSlice.disable_deblocking_filter_idc); // ue(v)

			if (OutSlice.disable_deblocking_filter_idc != 1)
			{
				Bitstream.Read(OutSlice.slice_alpha_c0_offset_div2); // se(v)
				Bitstream.Read(OutSlice.slice_beta_offset_div2);     // se(v)
			}
		}

		if (CurrentPPS.num_slice_groups_minus1 > 0
			&& CurrentPPS.slice_group_map_type >=3
			&& CurrentPPS.slice_group_map_type <= 5)
		{
			const uint32 NumBits = FMath::CeilLogTwo((CurrentPPS.pic_size_in_map_units_minus1 + 1) / ( CurrentPPS.slice_group_change_rate_minus1 + 2) );
			Bitstream.ReadBits(OutSlice.slice_group_change_cycle, NumBits); // u(v)
		}
		
		return EAVResult::Success;
	}

} // namespace UE::AVCodecCore::H264
