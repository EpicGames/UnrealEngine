// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/CodecUtils/CodecUtilsH265.h"

#include "Video/VideoPacket.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH265.h"

FH265ProfileDefinition GH265ProfileDefinitions[static_cast<uint8>(EH265Profile::MAX)] = {
	{ EH265Profile::Auto, UE::AVCodecCore::H265::EH265ProfileIDC::Auto, UE::AVCodecCore::H265::EH265ConstraintFlag::None, TEXT("Auto") }
};

namespace UE::AVCodecCore::H265
{
	FAVResult FindNALUs(FVideoPacket const& InPacket, TArray<FNaluH265>& FoundNalus)
	{
		FBitstreamReader Reader(InPacket.DataPtr.Get(), InPacket.DataSize);
		if (Reader.NumBytesRemaining() < 3)
		{
			return FAVResult(EAVResult::Warning, TEXT("Bitstream not long enough to hold a NALU"), TEXT("H265"));
		}

		const TArrayView64<uint8> Data = InPacket.GetData();
		
		// Skip over stream in intervals of 3 until Data[i + 2] is either 0 or 1
		for (int64 i = 0; i < Data.Num() - 2;)
		{
			if (Data[i + 2] > 1)
			{
				i += 3;
			}
			else if (Data[i + 2] == 1)
			{
				if (Data[i + 1] == 0 && Data[i] == 0)
				{
					// Found start sequence of NalU but we don't know if it has a 3 or 4 byte start code so we check.
                    FNaluH265 NalInfo = { (uint64)i, 0, 3, ENaluType::UNSPECIFIED, 0, 0, nullptr };
					if (NalInfo.Start > 0 && Data[NalInfo.Start - 1] == 0)
					{
						++NalInfo.StartCodeSize;
						--NalInfo.Start;
					}

					FBitstreamReader Bitstream(&Data[NalInfo.Start + NalInfo.StartCodeSize], 1);
					verifyf(Bitstream.ReadBits(1) == 0, TEXT("Forbidden Zero bit not Zero in NAL Header"));

                    NalInfo.Type = (ENaluType)Bitstream.ReadBits(6);
                    NalInfo.NuhLayerId = Bitstream.ReadBits(6);
                    NalInfo.NuhTemporalIdPlus1 = Bitstream.ReadBits(3);

                    NalInfo.Data = &Data[NalInfo.Start + NalInfo.StartCodeSize + 2];

					// Update length of previous entry.
					if (FoundNalus.Num() > 0)
					{
						FoundNalus.Last().Size = NalInfo.Start - (FoundNalus.Last().Start + FoundNalus.Last().StartCodeSize);
					}

					FoundNalus.Add(NalInfo);
				}

				i += 3;
			}
			else
			{
				++i;
			}
		}

		if (FoundNalus.Num() == 0)
		{
			return FAVResult(EAVResult::PendingInput, TEXT("no NALUs found in BitDataStream"), TEXT("BitstreamParserH265"));
		}

		// Last Nal size is the remaining size of the bitstream minus a trailing zero byte
		FoundNalus.Last().Size = Data.Num() - (FoundNalus.Last().Start + FoundNalus.Last().StartCodeSize);

	#if !IS_PROGRAM

		FAVResult::Log(EAVResult::Success, FString::Printf(TEXT("FAVExtension::TransformConfig found %d NALUs in bitdatastream"), FoundNalus.Num()));

		for (const FNaluH265& NalUInfo : FoundNalus)
		{
			FAVResult::Log(EAVResult::Success, FString::Printf(TEXT("Found NALU at %llu size %llu with type %u"), NalUInfo.Start, NalUInfo.Size, (uint8)NalUInfo.Type));
		}

	#endif //!IS_PROGRAM

		return EAVResult::Success;
	}

    FAVResult ParseVPS(FBitstreamReader& Bitstream, FNaluH265 const& InNaluInfo, TMap<uint32, VPS_t>& OutMapVPS)
    {
        FNalu::U<4> vps_video_parameter_set_id;
		Bitstream.Read(vps_video_parameter_set_id); // u(4)

		// Find VPS at current ID or set it to the map
		VPS_t& OutVPS = OutMapVPS.FindOrAdd(vps_video_parameter_set_id);

        OutVPS.vps_video_parameter_set_id = vps_video_parameter_set_id;

        Bitstream.Read(OutVPS.vps_base_layer_internal_flag,  // u(1)
                       OutVPS.vps_base_layer_available_flag, // u(1)
                       OutVPS.vps_max_layers_minus1,         // u(6)
                       OutVPS.vps_max_sub_layers_minus1,     // u(3)
                       OutVPS.vps_temporal_id_nesting_flag); // u(1)

        FNalu::U<16> vps_reserved_0xffff_16bits;
        Bitstream.Read(vps_reserved_0xffff_16bits); // u(16)

        OutVPS.profile_tier_level.Parse(1, OutVPS.vps_max_sub_layers_minus1, Bitstream);

        Bitstream.Read(OutVPS.vps_sub_layer_ordering_info_present_flag);

        OutVPS.vps_max_dec_pic_buffering_minus1.SetNumUninitialized(OutVPS.vps_max_sub_layers_minus1 + 1);
		OutVPS.vps_max_num_reorder_pics.SetNumUninitialized(OutVPS.vps_max_sub_layers_minus1 + 1);
		OutVPS.vps_max_latency_increase_plus1.SetNumUninitialized(OutVPS.vps_max_sub_layers_minus1 + 1);
		for (uint8 i = (OutVPS.vps_sub_layer_ordering_info_present_flag ? 0u : (uint8)OutVPS.vps_max_sub_layers_minus1); i <= OutVPS.vps_max_sub_layers_minus1; i++)
		{
			Bitstream.Read(
				OutVPS.vps_max_dec_pic_buffering_minus1[i], // ue(v)
				OutVPS.vps_max_num_reorder_pics[i],         // ue(v)
				OutVPS.vps_max_latency_increase_plus1[i]);  // ue(v)
		}
    
        Bitstream.Read(
			OutVPS.vps_max_layer_id,           // u(6)
			OutVPS.vps_num_layer_sets_minus1); // ue(v)

        OutVPS.layer_id_included_flag.SetNumZeroed(OutVPS.vps_num_layer_sets_minus1 + 1);
		for (uint8 i = 1; i <= OutVPS.vps_num_layer_sets_minus1; i++)
		{
            OutVPS.layer_id_included_flag[i].SetNumUninitialized(OutVPS.vps_max_layer_id + 1);
			for (uint8 j = 0; j <= OutVPS.vps_max_layer_id; j++)
			{
				Bitstream.Read(OutVPS.layer_id_included_flag[i][j]); // u(1)
			}
		}

        Bitstream.Read(OutVPS.vps_timing_info_present_flag); // u(1)

        if (OutVPS.vps_timing_info_present_flag)
		{
			Bitstream.Read(
				OutVPS.vps_num_units_in_tick,                // u(32)
				OutVPS.vps_time_scale,                       // u(32)
                OutVPS.vps_poc_proportional_to_timing_flag); // u(1)

			if (OutVPS.vps_poc_proportional_to_timing_flag)
			{
				Bitstream.Read(OutVPS.vps_num_ticks_poc_diff_one_minus1); // ue(v)
			}

			Bitstream.Read(OutVPS.vps_num_hrd_parameters); // ue(v)

            OutVPS.hrd_layer_set_idx.SetNumZeroed(OutVPS.vps_num_hrd_parameters);
            OutVPS.cprms_present_flag.SetNumZeroed(OutVPS.vps_num_hrd_parameters);
            OutVPS.hrd_parameters.SetNumZeroed(OutVPS.vps_num_hrd_parameters);
			for (uint32 i = 0; i < OutVPS.vps_num_hrd_parameters; i++)
			{
				Bitstream.Read(OutVPS.hrd_layer_set_idx[i]); // ue(v)
				if (i > 0)
				{
					Bitstream.Read(OutVPS.cprms_present_flag[i]); // u(1)
				}
				OutVPS.hrd_parameters[i].Parse(OutVPS.cprms_present_flag[i], OutVPS.vps_max_sub_layers_minus1, Bitstream);
			}
		}

		Bitstream.Read(OutVPS.vps_extension_flag);
		if (OutVPS.vps_extension_flag)
		{
			// TODO (aidan) handle VPS extentions ignoring for now
		}

		return EAVResult::Success;
    }

    void profile_tier_level_t::Parse(uint8 const& profilePresentFlag, uint8 maxNumSubLayersMinus1, FBitstreamReader& Bitstream)
    {
        if (profilePresentFlag)
		{
			Bitstream.Read(general_profile_space, // u(2)
				           general_tier_flag,     // u(1)
				           general_profile_idc);  // u(5)

            general_profile_compatibility_flag.SetNumUninitialized(32);
            for(size_t j = 0; j < 32; j++)
            {
                Bitstream.Read(general_profile_compatibility_flag[j]); // u(1)
            }

            Bitstream.Read(general_progressive_source_flag,     // u(1)
				           general_interlaced_source_flag,      // u(1)
				           general_non_packed_constraint_flag,  // u(1)
                           general_frame_only_constraint_flag); // u(1)
            if(general_profile_idc == EH265ProfileIDC::FormatRangeExtensions || general_profile_compatibility_flag[(uint8)(uint8)EH265ProfileIDC::FormatRangeExtensions] ||
               general_profile_idc == EH265ProfileIDC::HighThroughput || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::HighThroughput] ||
               general_profile_idc == EH265ProfileIDC::MultiViewMain || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::MultiViewMain] ||
               general_profile_idc == EH265ProfileIDC::ScalableMain || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::ScalableMain] ||
               general_profile_idc == EH265ProfileIDC::ThreeDimensionalMain || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::ThreeDimensionalMain] ||
               general_profile_idc == EH265ProfileIDC::ScreenContentCoding || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::ScreenContentCoding] ||
               general_profile_idc == EH265ProfileIDC::ScalableRangeExtensions || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::ScalableRangeExtensions] ||
               general_profile_idc == EH265ProfileIDC::HighThroughputScreenContentCoding || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::HighThroughputScreenContentCoding])
            {
                Bitstream.Read(general_max_12bit_constraint_flag,        // u(1)
                               general_max_10bit_constraint_flag,        // u(1)
                               general_max_8bit_constraint_flag,         // u(1)
                               general_max_422chroma_constraint_flag,    // u(1)
                               general_max_420chroma_constraint_flag,    // u(1)
                               general_max_monochrome_constraint_flag,   // u(1)
                               general_intra_constraint_flag,            // u(1)
                               general_one_picture_only_constraint_flag, // u(1)
                               general_lower_bit_rate_constraint_flag);  // u(1)

                if(general_profile_idc == EH265ProfileIDC::HighThroughput || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::HighThroughput] ||
                   general_profile_idc == EH265ProfileIDC::ScreenContentCoding || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::ScreenContentCoding] ||
                   general_profile_idc == EH265ProfileIDC::ScalableRangeExtensions || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::ScalableRangeExtensions] ||
                   general_profile_idc == EH265ProfileIDC::HighThroughputScreenContentCoding || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::HighThroughputScreenContentCoding])
                {
                    Bitstream.Read(general_max_14bit_constraint_flag); // u(1)
                    Bitstream.SkipBits(33);
                }
                else
                {
                    Bitstream.SkipBits(34);
                }
            }
            else if(general_profile_idc == EH265ProfileIDC::Main10 || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::Main10])
            {
                Bitstream.SkipBits(7);
				Bitstream.Read(general_one_picture_only_constraint_flag); // u(1)
				Bitstream.SkipBits(35);
            }
            else
            {
                Bitstream.SkipBits(43);
            }

            if(general_profile_idc == EH265ProfileIDC::Main || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::Main] ||
               general_profile_idc == EH265ProfileIDC::Main10 || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::Main10] ||
               general_profile_idc == EH265ProfileIDC::MainStillPicture || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::MainStillPicture] ||
               general_profile_idc == EH265ProfileIDC::FormatRangeExtensions || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::FormatRangeExtensions] ||
               general_profile_idc == EH265ProfileIDC::HighThroughput || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::HighThroughput] ||
               general_profile_idc == EH265ProfileIDC::ScreenContentCoding || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::ScreenContentCoding] ||
               general_profile_idc == EH265ProfileIDC::HighThroughputScreenContentCoding || general_profile_compatibility_flag[(uint8)EH265ProfileIDC::HighThroughputScreenContentCoding])
            {
                Bitstream.Read(general_inbld_flag); // u(1)
            }
            else
            {
                Bitstream.SkipBits(1);
            }
		}

        Bitstream.Read(general_level_idc); // u(8)

		sub_layers.SetNumZeroed(maxNumSubLayersMinus1 + 1);
		for (uint8 i = 0; i < maxNumSubLayersMinus1; i++)
		{
			Bitstream.Read(sub_layers[i].sub_layer_profile_present_flag, // u(1)
					       sub_layers[i].sub_layer_level_present_flag);  // u(1)
		}

		if (maxNumSubLayersMinus1 > 0)
		{
			for (uint8 i = maxNumSubLayersMinus1; i < 8; i++)
			{
				Bitstream.SkipBits(2); // u(2)
			}
		}

		for (uint8 i = 0; i < maxNumSubLayersMinus1; i++)
		{
			if (sub_layers[i].sub_layer_profile_present_flag)
			{
				Bitstream.Read(sub_layers[i].sub_layer_profile_space, // u(2) 
					           sub_layers[i].sub_layer_tier_flag,     // u(1)
					           sub_layers[i].sub_layer_profile_idc);  // u(5)

                sub_layers[i].sub_layer_profile_compatibility_flag.SetNumUninitialized(32);
                for(uint8 j = 0; j < 32; j++)
                {
                    Bitstream.Read(sub_layers[i].sub_layer_profile_compatibility_flag[j]); // u(1)
                }

                Bitstream.Read(sub_layers[i].sub_layer_progressive_source_flag,     // u(1) 
        			           sub_layers[i].sub_layer_interlaced_source_flag,      // u(1)
					           sub_layers[i].sub_layer_non_packed_constraint_flag,  // u(1)
                               sub_layers[i].sub_layer_frame_only_constraint_flag); // u(1)

                if(sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::FormatRangeExtensions || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::FormatRangeExtensions] ||
                   sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::HighThroughput || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::HighThroughput] ||
                   sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::MultiViewMain || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::MultiViewMain] ||
                   sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::ScalableMain || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::ScalableMain] ||
                   sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::ThreeDimensionalMain || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::ThreeDimensionalMain] ||
                   sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::ScreenContentCoding || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::ScreenContentCoding] ||
                   sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::ScalableRangeExtensions || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::ScalableRangeExtensions] ||
                   sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::HighThroughputScreenContentCoding || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::HighThroughputScreenContentCoding])
                {
                    Bitstream.Read(sub_layers[i].sub_layer_max_12bit_constraint_flag,        // u(1)
                                   sub_layers[i].sub_layer_max_10bit_constraint_flag,        // u(1)
                                   sub_layers[i].sub_layer_max_8bit_constraint_flag,         // u(1)
                                   sub_layers[i].sub_layer_max_422chroma_constraint_flag,    // u(1)
                                   sub_layers[i].sub_layer_max_420chroma_constraint_flag,    // u(1)
                                   sub_layers[i].sub_layer_max_monochrome_constraint_flag,   // u(1)
                                   sub_layers[i].sub_layer_intra_constraint_flag,            // u(1)
                                   sub_layers[i].sub_layer_one_picture_only_constraint_flag, // u(1)
                                   sub_layers[i].sub_layer_lower_bit_rate_constraint_flag);  // u(1)

                    if(sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::HighThroughput || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::HighThroughput] ||
                       sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::ScreenContentCoding || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::ScreenContentCoding] ||
                       sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::ScalableRangeExtensions || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::ScalableRangeExtensions] ||
                       sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::HighThroughputScreenContentCoding || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::HighThroughputScreenContentCoding])
                    {
                        Bitstream.Read(sub_layers[i].sub_layer_max_14bit_constraint_flag); // u(1)
                        Bitstream.SkipBits(33);
                    }
                    else
                    {
                        Bitstream.SkipBits(34);
                    }                        
                }
                else if(sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::Main10 || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::Main10])
                {
                    Bitstream.SkipBits(7);
				    Bitstream.Read(sub_layers[i].sub_layer_one_picture_only_constraint_flag); // u(1)
				    Bitstream.SkipBits(35);
                }
                else
                {
                    Bitstream.SkipBits(43);
                }

                if(sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::Main || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::Main] ||
                   sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::Main10 || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::Main10] ||
                   sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::MainStillPicture || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::MainStillPicture] ||
                   sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::FormatRangeExtensions || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::FormatRangeExtensions] ||
                   sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::HighThroughput || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::HighThroughput] ||
                   sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::ScreenContentCoding || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::ScreenContentCoding] ||
                   sub_layers[i].sub_layer_profile_idc == EH265ProfileIDC::HighThroughputScreenContentCoding || sub_layers[i].sub_layer_profile_compatibility_flag[(uint8)EH265ProfileIDC::HighThroughputScreenContentCoding])
                {
                    Bitstream.Read(sub_layers[i].sub_layer_inbld_flag); // u(1)
                }
                else
                {
                    Bitstream.SkipBits(1); // u(1)
                }
			}

			if (sub_layers[i].sub_layer_level_present_flag)
			{
				Bitstream.Read(sub_layers[i].sub_layer_level_idc);
			}
        }
    }

    void hrd_parameters_t::Parse(uint8 const& commonInfPresentFlag, uint8 const& maxNumSubLayersMinus1, FBitstreamReader& Bitstream)
	{
		if (commonInfPresentFlag)
		{
			Bitstream.Read(
				nal_hrd_parameters_present_flag,
				vcl_hrd_parameters_present_flag);

			if (nal_hrd_parameters_present_flag || vcl_hrd_parameters_present_flag)
			{
				Bitstream.Read(sub_pic_hrd_params_present_flag);
				if (sub_pic_hrd_params_present_flag)
				{
					Bitstream.Read(
						tick_divisor_minus2,
						du_cpb_removal_delay_increment_length_minus1,
						sub_pic_cpb_params_in_pic_timing_sei_flag,
						dpb_output_delay_du_length_minus1);
				}

				Bitstream.Read(
					bit_rate_scale,
					cpb_size_scale);

				if (sub_pic_hrd_params_present_flag)
				{
					Bitstream.Read(cpb_size_du_scale);
				}

				Bitstream.Read(
					initial_cpb_removal_delay_length_minus1,
					au_cpb_removal_delay_length_minus1,
					dpb_output_delay_length_minus1);
			}
		}

		sub_layers.SetNumZeroed(maxNumSubLayersMinus1 + 1);

		for (uint8 i = 0; i <= maxNumSubLayersMinus1; i++)
		{
			Bitstream.Read(sub_layers[i].fixed_pic_rate_general_flag);
			if (!sub_layers[i].fixed_pic_rate_general_flag)
			{
				Bitstream.Read(sub_layers[i].fixed_pic_rate_within_cvs_flag);
			}

			if (sub_layers[i].fixed_pic_rate_within_cvs_flag)
			{
				Bitstream.Read(sub_layers[i].elemental_duration_in_tc_minus1);
			}
			else
			{
				Bitstream.Read(sub_layers[i].low_delay_hrd_flag);
			}

			if (!sub_layers[i].low_delay_hrd_flag)
			{
				Bitstream.Read(sub_layers[i].cpb_cnt_minus1);
			}

			if (nal_hrd_parameters_present_flag)
			{
				sub_layers[i].Parse(sub_pic_hrd_params_present_flag, sub_layers[i].cpb_cnt_minus1 + 1, Bitstream);
			}

			if (vcl_hrd_parameters_present_flag)
			{
				sub_layers[i].Parse(sub_pic_hrd_params_present_flag, sub_layers[i].cpb_cnt_minus1 + 1, Bitstream);
			}
		}
	}


    void hrd_parameters_t::sub_layer_t::Parse(uint8 const& in_sub_pic_hrd_params_present_flag, uint8 const& CpbCnt, FBitstreamReader& Bitstream)
	{
		sub_layer_hrd_parameters.SetNumZeroed(CpbCnt);

		for (uint8 i = 0; i < CpbCnt; i++)
		{
			Bitstream.Read(
				sub_layer_hrd_parameters[i].bit_rate_value_minus1,
				sub_layer_hrd_parameters[i].cpb_size_value_minus1);
			if (in_sub_pic_hrd_params_present_flag)
			{
				Bitstream.Read(
					sub_layer_hrd_parameters[i].cpb_size_du_value_minus1,
					sub_layer_hrd_parameters[i].bit_rate_du_value_minus1);
			}
			Bitstream.Read(sub_layer_hrd_parameters[i].cbr_flag);
		}
	}

    FAVResult ParseSPS(FBitstreamReader& Bitstream, FNaluH265 const& InNaluInfo, TMap<uint32, VPS_t> const& InMapVPS, TMap<uint32, SPS_t>& OutMapSPS)
    {
        FNalu::U<4> sps_video_parameter_set_id;
        Bitstream.Read(sps_video_parameter_set_id); // u(4)

        FNalu::U<3> sps_max_sub_layers_minus1;
        Bitstream.Read(sps_max_sub_layers_minus1); // u(3)

        FNalu::U<1> sps_temporal_id_nesting_flag;
        Bitstream.Read(sps_temporal_id_nesting_flag); // u(1)

        profile_tier_level_t profile_tier_level;
        profile_tier_level.Parse(1, sps_max_sub_layers_minus1, Bitstream);

        FNalu::UE sps_seq_parameter_set_id;
        Bitstream.Read(sps_seq_parameter_set_id); // ue(v)

        SPS_t& OutSPS = OutMapSPS.FindOrAdd(sps_seq_parameter_set_id);

        OutSPS.sps_video_parameter_set_id = sps_video_parameter_set_id;
        OutSPS.sps_max_sub_layers_minus1 = sps_max_sub_layers_minus1;
        OutSPS.sps_temporal_id_nesting_flag = sps_temporal_id_nesting_flag;
        OutSPS.profile_tier_level = profile_tier_level;
        OutSPS.sps_seq_parameter_set_id = sps_seq_parameter_set_id;

        Bitstream.Read(OutSPS.chroma_format_idc); // ue(v)
        if(OutSPS.chroma_format_idc == 3)
        {
            Bitstream.Read(OutSPS.separate_colour_plane_flag); // u(1)
        }

        Bitstream.Read(OutSPS.pic_width_in_luma_samples,  // ue(v)
                       OutSPS.pic_height_in_luma_samples, // ue(v)
                       OutSPS.conformance_window_flag);   // u(1)

        if(OutSPS.conformance_window_flag)
        {
            Bitstream.Read(OutSPS.conf_win.left_offset,    // ue(v)
				           OutSPS.conf_win.right_offset,   // ue(v)
				           OutSPS.conf_win.top_offset,     // ue(v)
				           OutSPS.conf_win.bottom_offset); // ue(v)
        }

        Bitstream.Read(OutSPS.bit_depth_luma_minus8,                     // ue(v)
			           OutSPS.bit_depth_chroma_minus8,                   // ue(v)
			           OutSPS.log2_max_pic_order_cnt_lsb_minus4,         // ue(v)
			           OutSPS.sps_sub_layer_ordering_info_present_flag); // u(1)

        OutSPS.sub_layer_ordering_infos.SetNumUninitialized(OutSPS.sps_max_sub_layers_minus1 + 1);
		for (uint8 i = (OutSPS.sps_sub_layer_ordering_info_present_flag ? 0u : (uint8)OutSPS.sps_max_sub_layers_minus1); i <= OutSPS.sps_max_sub_layers_minus1; i++)
		{
			Bitstream.Read(
				OutSPS.sub_layer_ordering_infos[i].sps_max_dec_pic_buffering_minus1, // ue(v)
				OutSPS.sub_layer_ordering_infos[i].sps_max_num_reorder_pics,         // ue(v)
				OutSPS.sub_layer_ordering_infos[i].sps_max_latency_increase_plus1);  // ue(v)
		}

        Bitstream.Read(OutSPS.log2_min_luma_coding_block_size_minus3,      // ue(v)
			           OutSPS.log2_diff_max_min_luma_coding_block_size,    // ue(v)
			           OutSPS.log2_min_luma_transform_block_size_minus2,   // ue(v)
			           OutSPS.log2_diff_max_min_luma_transform_block_size, // ue(v)
			           OutSPS.max_transform_hierarchy_depth_inter,         // ue(v)
			           OutSPS.max_transform_hierarchy_depth_intra,         // ue(v)
			           OutSPS.scaling_list_enabled_flag);                  // u(1)

        if (OutSPS.scaling_list_enabled_flag)
		{
			Bitstream.Read(OutSPS.sps_scaling_list_data_present_flag); // u(1)
			if (OutSPS.sps_scaling_list_data_present_flag)
			{
				OutSPS.scaling_list_data.Parse(Bitstream);
			}
		}

        Bitstream.Read(OutSPS.amp_enabled_flag,                    // u(1)
			           OutSPS.sample_adaptive_offset_enabled_flag, // u(1)
			           OutSPS.pcm_enabled_flag);                   // u(1)

        if(OutSPS.pcm_enabled_flag)
        {
            Bitstream.Read(OutSPS.pcm_sample_bit_depth_luma_minus1,             // u(4)
				           OutSPS.pcm_sample_bit_depth_chroma_minus1,           // u(4)
				           OutSPS.log2_min_pcm_luma_coding_block_size_minus3,   // ue(v)
				           OutSPS.log2_diff_max_min_pcm_luma_coding_block_size, // ue(v)
				           OutSPS.pcm_loop_filter_disabled_flag);               // u(1)
        }

        Bitstream.Read(OutSPS.num_short_term_ref_pic_sets); // ue(v)
		OutSPS.short_term_ref_pic_sets.SetNumZeroed(OutSPS.num_short_term_ref_pic_sets + 1); // We add one here as a slice can append a picture to this list so index "num_short_term_ref_pic_sets" is valid
		for (uint8 i = 0; i < OutSPS.num_short_term_ref_pic_sets; i++)
		{
			OutSPS.short_term_ref_pic_sets[i].Parse(i, OutSPS.short_term_ref_pic_sets, Bitstream);
			OutSPS.short_term_ref_pic_sets[i].CalculateValues(i, OutSPS.short_term_ref_pic_sets);
		}

		Bitstream.Read(OutSPS.long_term_ref_pics_present_flag); // u(1)
		if (OutSPS.long_term_ref_pics_present_flag)
		{
			Bitstream.Read(OutSPS.num_long_term_ref_pics_sps); // ue(v)
			OutSPS.lt_ref_pic_poc_lsb_sps.SetNumZeroed(OutSPS.num_long_term_ref_pics_sps);
            OutSPS.used_by_curr_pic_lt_sps_flag.SetNumZeroed(OutSPS.num_long_term_ref_pics_sps);
			for (uint8 i = 0; i < OutSPS.num_long_term_ref_pics_sps; i++)
			{
                Bitstream.ReadBits(OutSPS.lt_ref_pic_poc_lsb_sps[i], OutSPS.log2_max_pic_order_cnt_lsb_minus4 + 4); // u(v)
		        Bitstream.Read(OutSPS.used_by_curr_pic_lt_sps_flag[i]);                                             // u(1)
			}
		}

		Bitstream.Read(
			OutSPS.sps_temporal_mvp_enabled_flag,       // u(1)
			OutSPS.strong_intra_smoothing_enabled_flag, // u(1)
			OutSPS.vui_parameters_present_flag);        // u(1)

		if (OutSPS.vui_parameters_present_flag)
		{
			OutSPS.vui_parameters.Parse(Bitstream, OutSPS.sps_max_sub_layers_minus1);
		}

		Bitstream.Read(OutSPS.sps_extension_present_flag);

		if (OutSPS.sps_extension_present_flag)
		{
			Bitstream.Read(
				OutSPS.sps_range_extension_flag,      // u(1)
				OutSPS.sps_multilayer_extension_flag, // u(1)
				OutSPS.sps_3d_extension_flag,         // u(1)
				OutSPS.sps_scc_extension_flag,        // u(1)
				OutSPS.sps_extension_4bits);          // u(4)
		}

		if (OutSPS.sps_range_extension_flag)
		{
			OutSPS.sps_range_extension.Parse(Bitstream);
		}

		if (OutSPS.sps_multilayer_extension_flag)
		{
			OutSPS.sps_multilayer_extension.Parse(Bitstream);
		}

		if (OutSPS.sps_3d_extension_flag)
		{
			OutSPS.sps_3d_extension.Parse(Bitstream);
		}

		if (OutSPS.sps_scc_extension_flag)
		{
			OutSPS.sps_scc_extension.Parse(OutSPS.chroma_format_idc, OutSPS.bit_depth_chroma_minus8, Bitstream);
		}

		if (OutSPS.sps_extension_4bits)
		{
			// TODO (aidan) read more extensions
		}



        return EAVResult::Success;
    }

    void scaling_list_data_t::Parse(FBitstreamReader& Bitstream)
    {
        SE scaling_list_delta_coef;
		for (uint8 sizeId = 0; sizeId < 4; sizeId++)
		{
			for (uint8 matrixId = 0; matrixId < 6; matrixId += (sizeId == 3) ? 3 : 1)
			{
				Bitstream.Read(scaling_list_pred_mode_flag[sizeId][matrixId]);
				if (!scaling_list_pred_mode_flag[sizeId][matrixId])
				{
					Bitstream.Read(scaling_list_pred_matrix_id_delta[sizeId][matrixId]);
				}
				else
				{
					int16 nextCoef = 8;
					uint8 coefNum = FMath::Min<uint8>(64u, (1 << (4 + (sizeId << 1))));

					if (sizeId > 1)
					{
						Bitstream.Read(scaling_list_dc_coef_minus8[sizeId - 2][matrixId]);
						nextCoef = scaling_list_dc_coef_minus8[sizeId - 2][matrixId] + 8;
					}

					if (sizeId == 0)
					{
						FMemory::Memset(ScalingList0[matrixId], 0, sizeof(ScalingList0[0]));
						for (uint8 i = 0; i < coefNum; i++)
						{
							Bitstream.Read(scaling_list_delta_coef);
							nextCoef = (nextCoef + scaling_list_delta_coef + 256) % 256;
							ScalingList0[matrixId][i] = nextCoef;
						}
					}
					else
					{
						FMemory::Memset(ScalingList1to3[sizeId - 1], 0, sizeof(ScalingList1to3[0]));
						for (uint8 i = 0; i < coefNum; i++)
						{
							Bitstream.Read(scaling_list_delta_coef);
							nextCoef = (nextCoef + scaling_list_delta_coef + 256) % 256;
							ScalingList1to3[sizeId - 1][matrixId][i] = nextCoef;
						}
					}
				}
			}
		}
    }

    void short_term_ref_pic_set_t::Parse(uint8 const& stRpsIdx, TArray<short_term_ref_pic_set_t> const& short_term_ref_pic_sets, FBitstreamReader& Bitstream)
	{
		if (stRpsIdx != 0)
		{
			Bitstream.Read(inter_ref_pic_set_prediction_flag); // u(1)
		}

		if (inter_ref_pic_set_prediction_flag == 1)
		{
			if (stRpsIdx == short_term_ref_pic_sets.Num())
			{
				Bitstream.Read(delta_idx_minus1); // ue(v)
			}

			Bitstream.Read(
				delta_rps_sign,        // u(1)
				abs_delta_rps_minus1); // ue(v)

			uint32 RefRpsIdx = stRpsIdx - (delta_idx_minus1 + 1);
			uint8 RefNumDeltaPocs = short_term_ref_pic_sets[RefRpsIdx].NumDeltaPocs;

            used_by_curr_pic_flags = new FNalu::U<1>[RefNumDeltaPocs];
            use_delta_flags = new FNalu::U<1>[RefNumDeltaPocs];
			// used_by_curr_pic_flags.SetNumUninitialized(RefNumDeltaPocs);
			// use_delta_flags.SetNumUninitialized(RefNumDeltaPocs);

			for (uint8 j = 0; j < RefNumDeltaPocs; j++)
			{
				Bitstream.Read(used_by_curr_pic_flags[j]); // u(1)
				if (!used_by_curr_pic_flags[j])
				{
					Bitstream.Read(use_delta_flags[j]); // u(1)
				}
				else
				{
					use_delta_flags[j] = 1; // when not present inferred as 1 according to (7.4.8)
				}
			}
		}
		else // inter_ref_pic_set_prediction_flag != 1
		{
			// Static Analysis complains about our reading of the arrays and that we could
			// be reading past the end of the buffer but this is not true
			#pragma warning(push)
			#pragma warning(disable : 6385)
			Bitstream.Read(
				num_negative_pics,  // ue(v)
				num_positive_pics); // ue(v)

            delta_poc_s0_minus1s = new FNalu::UE[num_negative_pics];
            used_by_curr_pic_s0_flags = new FNalu::U<1>[num_negative_pics];
			// delta_poc_s0_minus1s.SetNumUninitialized(num_negative_pics);
			// used_by_curr_pic_s0_flags.SetNumUninitialized(num_negative_pics);
			for (uint8 i = 0; i < num_negative_pics; i++)
			{
				Bitstream.Read(
					delta_poc_s0_minus1s[i],       // ue(v)
					used_by_curr_pic_s0_flags[i]); // u(1)
			}

            delta_poc_s1_minus1s = new FNalu::UE[num_positive_pics];
            used_by_curr_pic_s1_flags = new FNalu::U<1>[num_positive_pics];
			// delta_poc_s1_minus1s.SetNumUninitialized(num_positive_pics);
			// used_by_curr_pic_s1_flags.SetNumUninitialized(num_positive_pics);
			for (uint8 i = 0; i < num_positive_pics; i++)
			{
				Bitstream.Read(
					delta_poc_s1_minus1s[i],       // ue(v)
					used_by_curr_pic_s1_flags[i]); // u(1)
			}
			#pragma warning(pop)
		}
	}

	void short_term_ref_pic_set_t::CalculateValues(uint8 const& stRpsIdx, TArray<short_term_ref_pic_set_t>& st_ref_pic_set)
	{
		if (inter_ref_pic_set_prediction_flag)
		{
			uint8 RefRpsIdx = stRpsIdx - (delta_idx_minus1 + 1);
			int16 deltaRps = (1 - 2 * delta_rps_sign) * (abs_delta_rps_minus1 + 1);
			int16 dPoc;

			uint8 i = 0;
			for (int16 j = st_ref_pic_set[RefRpsIdx].NumPositivePics - 1; j >= 0; j--)
			{
				dPoc = st_ref_pic_set[RefRpsIdx].DeltaPocS1[j] + deltaRps;
				if (dPoc < 0 && use_delta_flags[st_ref_pic_set[RefRpsIdx].NumNegativePics + j])
				{
					DeltaPocS0[i] = dPoc;
					UsedByCurrPicS0[i++] = used_by_curr_pic_flags[st_ref_pic_set[RefRpsIdx].NumNegativePics + j];
				}
			}

			if (deltaRps < 0 && use_delta_flags[st_ref_pic_set[RefRpsIdx].NumDeltaPocs])
			{
				DeltaPocS0[i] = deltaRps;
				UsedByCurrPicS0[i++] = used_by_curr_pic_flags[st_ref_pic_set[RefRpsIdx].NumDeltaPocs];
			}

			for (int16 j = 0; j < st_ref_pic_set[RefRpsIdx].NumNegativePics; j++)
			{
				dPoc = st_ref_pic_set[RefRpsIdx].DeltaPocS0[j] + deltaRps;
				if (dPoc < 0 && use_delta_flags[j])
				{
					DeltaPocS0[i] = dPoc;
					UsedByCurrPicS0[i++] = used_by_curr_pic_flags[j];
				}
			}
			NumNegativePics = i;

			i = 0;
			for (int16 j = st_ref_pic_set[RefRpsIdx].NumNegativePics - 1; j >= 0; j--)
			{
				dPoc = st_ref_pic_set[RefRpsIdx].DeltaPocS0[j] + deltaRps;
				if (dPoc > 0 && use_delta_flags[j])
				{
					DeltaPocS1[i] = dPoc;
					UsedByCurrPicS1[i++] = used_by_curr_pic_flags[j];
				}
			}

			if (deltaRps > 0 && use_delta_flags[st_ref_pic_set[RefRpsIdx].NumDeltaPocs])
			{
				DeltaPocS1[i] = deltaRps;
				UsedByCurrPicS1[i++] = used_by_curr_pic_flags[st_ref_pic_set[RefRpsIdx].NumDeltaPocs];
			}

			for (int16 j = 0; j < st_ref_pic_set[RefRpsIdx].NumPositivePics; j++)
			{
				dPoc = st_ref_pic_set[RefRpsIdx].DeltaPocS1[j] + deltaRps;
				if (dPoc > 0 && use_delta_flags[st_ref_pic_set[RefRpsIdx].NumNegativePics + j])
				{
					DeltaPocS1[i] = dPoc;
					UsedByCurrPicS1[i++] = used_by_curr_pic_flags[st_ref_pic_set[RefRpsIdx].NumNegativePics + j];
				}
			}
			NumPositivePics = i;
		}
		else
		{
			NumNegativePics = num_negative_pics;
			for (uint8 i = 0; i < NumNegativePics; i++)
			{
				UsedByCurrPicS0[i] = used_by_curr_pic_s0_flags[i];
				DeltaPocS0[i] = i == 0 ? 0 : DeltaPocS0[i-1];
				DeltaPocS0[i] -= ( delta_poc_s0_minus1s[i] + 1);
			}

			NumPositivePics = num_positive_pics;
			for (uint8 i = 0; i < NumPositivePics; i++)
			{
				UsedByCurrPicS0[i] = used_by_curr_pic_s1_flags[i];
				DeltaPocS1[i] = i == 0 ? 0 : DeltaPocS1[i-1];
				DeltaPocS1[i] += ( delta_poc_s1_minus1s[i] + 1);
			}

			NumDeltaPocs = NumNegativePics + NumPositivePics;
		}
	}

    void SPS_t::vui_parameters_t::Parse(FBitstreamReader& Bitstream, uint8 const& in_sps_max_sub_layers_minus1)
    {
        Bitstream.Read(aspect_ratio_info_present_flag); // u(1)
		if (aspect_ratio_info_present_flag)
		{
			Bitstream.Read(aspect_ratio_idc); // u(8)
			if (aspect_ratio_idc == 255)
			{
				Bitstream.Read(
					sar_width,   // u(16)
					sar_height); // u(16)
			}
		}

		Bitstream.Read(overscan_info_present_flag); // u(1)
		if (overscan_info_present_flag)
		{
			Bitstream.Read(overscan_appropriate_flag); // u(1)
		}

		Bitstream.Read(video_signal_type_present_flag); // u(1)
		if (video_signal_type_present_flag)
		{
			Bitstream.Read(
				video_format,                     // u(3)
				video_full_range_flag,            // u(1)
				colour_description_present_flag); // u(1)

			if (colour_description_present_flag)
			{
				Bitstream.Read(
					colour_primaries,         // u(8)
					transfer_characteristics, // u(8)
					matrix_coeffs);           // u(8)
			}
		}

		Bitstream.Read(chroma_loc_info_present_flag); // u(1)
		if (chroma_loc_info_present_flag)
		{
			Bitstream.Read(
				chroma_sample_loc_type_top_field,     // ue(v)
				chroma_sample_loc_type_bottom_field); // ue(v)
		}

		Bitstream.Read(
			neutral_chroma_indication_flag, // u(1)
			field_seq_flag,                 // u(1)
			frame_field_info_present_flag,  // u(1)
			default_display_window_flag);   // u(1)
        
		if (default_display_window_flag)
		{
			Bitstream.Read(
				def_disp_win_left_offset,    // ue(v)
				def_disp_win_right_offset,   // ue(v)
				def_disp_win_top_offset,     // ue(v)
				def_disp_win_bottom_offset); // ue(v)
		}

		Bitstream.Read(vui_timing_info_present_flag); // u(1)
		if (vui_timing_info_present_flag)
		{
			Bitstream.Read(
				vui_num_units_in_tick,                // u(32)
				vui_time_scale,                       // u(32)
				vui_poc_proportional_to_timing_flag); // u(1)

			if (vui_poc_proportional_to_timing_flag)
			{
				Bitstream.Read(vui_num_ticks_poc_diff_one_minus1); // ue(v)
			}

			Bitstream.Read(vui_hrd_parameters_present_flag); // u(1)
			if (vui_hrd_parameters_present_flag)
			{
				hrd_parameters.Parse( 1, in_sps_max_sub_layers_minus1, Bitstream );
			}
		}

		Bitstream.Read(bitstream_restriction_flag); // u(1)
		if (bitstream_restriction_flag)
		{
			Bitstream.Read(
				tiles_fixed_structure_flag,              // u(1)
				motion_vectors_over_pic_boundaries_flag, // u(1)
				restricted_ref_pic_lists_flag,           // u(1)
				min_spatial_segmentation_idc,            // ue(v)
				max_bytes_per_pic_denom,                 // ue(v)
				max_bits_per_min_cu_denom,               // ue(v)
				log2_max_mv_length_horizontal,           // ue(v)
				log2_max_mv_length_vertical);            // ue(v)
		}
    }

    void SPS_t::sps_range_extension_t::Parse(FBitstreamReader& Bitstream)
    {
        Bitstream.Read(
			transform_skip_rotation_enabled_flag,    // u(1)
			transform_skip_context_enabled_flag,     // u(1)
			implicit_rdpcm_enabled_flag,             // u(1)
			explicit_rdpcm_enabled_flag,             // u(1)
			extended_precision_processing_flag,      // u(1)
			intra_smoothing_disabled_flag,           // u(1)
			high_precision_offsets_enabled_flag,     // u(1)
			persistent_rice_adaptation_enabled_flag, // u(1)
			cabac_bypass_alignment_enabled_flag);    // u(1)
    }

    void SPS_t::sps_scc_extension_t::Parse(uint32 const& in_chroma_format_idc, uint32 const& in_bit_depth_chroma_minus8, FBitstreamReader& Bitstream)
	{
		Bitstream.Read(
			sps_curr_pic_ref_enabled_flag, // u(1)
			palette_mode_enabled_flag);    // u(1)

		if (palette_mode_enabled_flag)
		{
			Bitstream.Read(
				palette_max_size,                                 // ue(v)
				delta_palette_max_predictor_size,                 // ue(v)
				sps_palette_predictor_initializers_present_flag); // u(1)

			if (sps_palette_predictor_initializers_present_flag)
			{
				Bitstream.Read(sps_num_palette_predictor_initializers_minus1); // ue(v)
				const uint8 numComps = (in_chroma_format_idc == 0) ? 1u : 3u;
				sps_palette_predictor_initializers.SetNumZeroed(numComps);
				for (uint8 comp = 0; comp < numComps; comp++)
				{
					sps_palette_predictor_initializers[comp].SetNumUninitialized(sps_num_palette_predictor_initializers_minus1 + 1);
					for (uint8 i = 0; i <= sps_num_palette_predictor_initializers_minus1; i++)
					{
						Bitstream.ReadBits(sps_palette_predictor_initializers[comp][i], in_bit_depth_chroma_minus8 + 8); // u(v)
					}
				}
			}
		}

		Bitstream.Read(
			motion_vector_resolution_control_idc,    // u(2)
			intra_boundary_filtering_disabled_flag); // u(1)
	}

    FAVResult ParsePPS(FBitstreamReader& Bitstream, FNaluH265 const& InNaluInfo, TMap<uint32, VPS_t> const& InMapVPS, TMap<uint32, SPS_t> const& InMapSPS, TMap<uint32, PPS_t>& OutMapPPS)
    {
        FNalu::UE pps_pic_parameter_set_id;
		Bitstream.Read(pps_pic_parameter_set_id); // ue(v)

		PPS_t& OutPPS = OutMapPPS.FindOrAdd(pps_pic_parameter_set_id);
		OutPPS.pps_pic_parameter_set_id = pps_pic_parameter_set_id;

        Bitstream.Read(OutPPS.pps_seq_parameter_set_id,              // ue(v)
                       OutPPS.dependent_slice_segments_enabled_flag, // u(1)
                       OutPPS.output_flag_present_flag,              // u(1)
                       OutPPS.num_extra_slice_header_bits,           // u(3)
                       OutPPS.sign_data_hiding_enabled_flag,         // u(1)
                       OutPPS.cabac_init_present_flag,               // u(1)
                       OutPPS.num_ref_idx_l0_default_active_minus1,  // ue(v)
                       OutPPS.num_ref_idx_l1_default_active_minus1,  // ue(v)
                       OutPPS.init_qp_minus26,                       // se(v)
                       OutPPS.constrained_intra_pred_flag,           // u(1)
                       OutPPS.transform_skip_enabled_flag,           // u(1)
                       OutPPS.cu_qp_delta_enabled_flag);             // u(1)
        
        if(OutPPS.cu_qp_delta_enabled_flag)
        {
            Bitstream.Read(OutPPS.diff_cu_qp_delta_depth);
        }

        Bitstream.Read(OutPPS.pps_cb_qp_offset);                         // se(v)
                       Bitstream.Read(OutPPS.pps_cr_qp_offset);                         // se(v)
                       Bitstream.Read(OutPPS.pps_slice_chroma_qp_offsets_present_flag); // u(1)
                       Bitstream.Read(OutPPS.weighted_pred_flag);                       // u(1)
                       Bitstream.Read(OutPPS.weighted_bipred_flag);                     // u(1)
                       Bitstream.Read(OutPPS.transquant_bypass_enabled_flag);           // u(1)
                       Bitstream.Read(OutPPS.tiles_enabled_flag);                       // u(1)
                       Bitstream.Read(OutPPS.entropy_coding_sync_enabled_flag);       // u(1)

        if(OutPPS.tiles_enabled_flag)
        {
            Bitstream.Read(OutPPS.num_tile_columns_minus1, // ue(v)
                           OutPPS.num_tile_rows_minus1,    // ue(v)
                           OutPPS.uniform_spacing_flag);   // u(1)    
            
            if(!OutPPS.uniform_spacing_flag)
            {
                OutPPS.column_width_minus1.SetNumUninitialized(OutPPS.num_tile_columns_minus1 + 1);
                for(size_t i = 0; i < OutPPS.num_tile_columns_minus1; i++)
                {
                    Bitstream.Read(OutPPS.column_width_minus1[i]); // ue(v)
                }

                OutPPS.row_height_minus1.SetNumUninitialized(OutPPS.num_tile_rows_minus1 + 1);
                for(size_t i = 0; i < OutPPS.num_tile_rows_minus1; i++)
                {
                    Bitstream.Read(OutPPS.row_height_minus1[i]); // ue(v)
                }

                Bitstream.Read(OutPPS.loop_filter_across_tiles_enabled_flag); // u(1)
            }
        }

        Bitstream.Read(OutPPS.pps_loop_filter_across_slices_enabled_flag, // u(1)
			           OutPPS.deblocking_filter_control_present_flag);    // u(1)

        if(OutPPS.deblocking_filter_control_present_flag)
        {
            Bitstream.Read(OutPPS.deblocking_filter_override_enabled_flag, // u(1)
			               OutPPS.pps_deblocking_filter_disabled_flag);    // u(1)
            
            if(!OutPPS.pps_deblocking_filter_disabled_flag)
            {
                Bitstream.Read(OutPPS.pps_beta_offset_div2, // se(v)
			                   OutPPS.pps_tc_offset_div2);  // se(v)
            }
        }

        Bitstream.Read(OutPPS.pps_scaling_list_data_present_flag); // u(1)
        if(OutPPS.pps_scaling_list_data_present_flag)
        {
            OutPPS.scaling_list_data.Parse(Bitstream);
        }

        Bitstream.Read(OutPPS.lists_modification_present_flag,             // u(1)
			           OutPPS.log2_parallel_merge_level_minus2,            // ue(v)
                       OutPPS.slice_segment_header_extension_present_flag, // u(1)
                       OutPPS.pps_extension_present_flag);                 // u(1)

        if(OutPPS.pps_extension_present_flag)
        {
            Bitstream.Read(OutPPS.pps_range_extension_flag,      // u(1)
				           OutPPS.pps_multilayer_extension_flag, // u(1)
				           OutPPS.pps_3d_extension_flag,         // u(1)
				           OutPPS.pps_scc_extension_flag,        // u(1)
				           OutPPS.pps_extension_4bits);          // u(4)
        }

        if (OutPPS.pps_range_extension_flag)
		{
			OutPPS.pps_range_extension.Parse(OutPPS.transform_skip_enabled_flag, Bitstream);
		}

		if (OutPPS.pps_multilayer_extension_flag)
		{
			OutPPS.pps_multilayer_extension.Parse(Bitstream);
		}

		if (OutPPS.pps_3d_extension_flag)
		{
			OutPPS.pps_3d_extension.Parse(Bitstream);
		}

		if (OutPPS.pps_scc_extension_flag)
		{
			OutPPS.pps_scc_extension.Parse(Bitstream);
		}
        
        return EAVResult::Success;
    }

    void PPS_t::pps_range_extension_t::Parse(uint8 const& in_transform_skip_enabled_flag, FBitstreamReader& Bitstream)
	{
		if (in_transform_skip_enabled_flag)
		{
			Bitstream.Read(log2_max_transform_skip_block_size_minus2);
		}

		Bitstream.Read(
			cross_component_prediction_enabled_flag,
			chroma_qp_offset_list_enabled_flag);

		if (chroma_qp_offset_list_enabled_flag)
		{
			Bitstream.Read(
				diff_cu_chroma_qp_offset_depth,
				chroma_qp_offset_list_len_minus1);

			cb_qp_offset_list.SetNumUninitialized(chroma_qp_offset_list_len_minus1 + 1);
			cr_qp_offset_list.SetNumUninitialized(chroma_qp_offset_list_len_minus1 + 1);
			for (uint8 i = 0; i <= chroma_qp_offset_list_len_minus1; i++)
			{
				Bitstream.Read(
					cb_qp_offset_list[i],
					cr_qp_offset_list[i]);
			}
		}

		Bitstream.Read(
			log2_sao_offset_scale_luma,
			log2_sao_offset_scale_chroma);
	}

	void PPS_t::pps_scc_extension_t::Parse(FBitstreamReader& Bitstream)
	{
		Bitstream.Read(
			pps_curr_pic_ref_enabled_flag,
			residual_adaptive_colour_transform_enabled_flag);

		if (residual_adaptive_colour_transform_enabled_flag)
		{
			Bitstream.Read(
				pps_slice_act_qp_offsets_present_flag,
				pps_act_y_qp_offset_plus5,
				pps_act_cb_qp_offset_plus5,
				pps_act_cr_qp_offset_plus3);
		}

		Bitstream.Read(pps_palette_predictor_initializers_present_flag);
		if (pps_palette_predictor_initializers_present_flag)
		{
			Bitstream.Read(pps_num_palette_predictor_initializers);
			if (pps_num_palette_predictor_initializers > 0)
			{
				Bitstream.Read(
					monochrome_palette_flag,
					luma_bit_depth_entry_minus8);

				if (!monochrome_palette_flag)
				{
					Bitstream.Read(chroma_bit_depth_entry_minus8);
				}

				uint8 numComps = monochrome_palette_flag ? 1 : 3;
				pps_palette_predictor_initializer.SetNumZeroed(numComps);
				for (uint8 comp = 0; comp < numComps; comp++)
				{
					pps_palette_predictor_initializer[comp].SetNumZeroed(pps_num_palette_predictor_initializers);
					for (uint8 i = 0; i < pps_num_palette_predictor_initializers; i++)
					{
						Bitstream.ReadBits(pps_palette_predictor_initializer[comp][i], (i >= 1) ? chroma_bit_depth_entry_minus8 + 8 : luma_bit_depth_entry_minus8 + 8);
					}
				}
			}
		}
	}

    FAVResult ParseSliceHeader(FBitstreamReader& Bitstream, FNaluH265 const& InNaluInfo, TMap<uint32, VPS_t> const& InMapVPS, TMap<uint32, SPS_t> const& InMapSPS, TMap<uint32, PPS_t> const& InMapPPS, Slice_t& OutSlice)
    {
        Bitstream.Read(OutSlice.first_slice_segment_in_pic_flag); // u(1)

        if (InNaluInfo.Type >= ENaluType::BLA_W_LP && InNaluInfo.Type <= ENaluType::RSV_IRAP_VCL23)
		{
			Bitstream.Read(OutSlice.no_output_of_prior_pics_flag); // u(1)
		}

        Bitstream.Read(OutSlice.slice_pic_parameter_set_id); // ue(v)
        
        const PPS_t* PPS = InMapPPS.Find(OutSlice.slice_pic_parameter_set_id);
        if(PPS == nullptr)
        {
            return FAVResult(EAVResult::Error, TEXT("Missing PPS (%d)"), TEXT("VT"), OutSlice.slice_pic_parameter_set_id);
        }
        
        const SPS_t* SPS = InMapSPS.Find(PPS->pps_seq_parameter_set_id);
        if(SPS == nullptr)
        {
            return FAVResult(EAVResult::Error, TEXT("Missing SPS (%d)"), TEXT("VT"), PPS->pps_seq_parameter_set_id);
        }
        
        const VPS_t* VPS = InMapVPS.Find(SPS->sps_video_parameter_set_id);
        if(VPS == nullptr)
        {
            return FAVResult(EAVResult::Error, TEXT("Missing VPS (%d)"), TEXT("VT"), SPS->sps_video_parameter_set_id);
        }

        const PPS_t& CurrentPPS = InMapPPS[OutSlice.slice_pic_parameter_set_id];
		const SPS_t& CurrentSPS = InMapSPS[CurrentPPS.pps_seq_parameter_set_id];
        const VPS_t& CurrentVPS = InMapVPS[CurrentSPS.sps_video_parameter_set_id];

        // Default values
        OutSlice.use_integer_mv_flag = CurrentSPS.sps_scc_extension.motion_vector_resolution_control_idc;
        OutSlice.slice_deblocking_filter_disabled_flag = CurrentPPS.pps_deblocking_filter_disabled_flag;
        OutSlice.slice_loop_filter_across_slices_enabled_flag = CurrentPPS.pps_deblocking_filter_disabled_flag;
        OutSlice.num_long_term_pics = 0;

        if(!OutSlice.first_slice_segment_in_pic_flag)
        {
            if(CurrentPPS.dependent_slice_segments_enabled_flag)
            {
                Bitstream.Read(OutSlice.dependent_slice_segment_flag); // u(1)
            }

            const uint32 MinCbLog2SizeY = CurrentSPS.log2_min_luma_coding_block_size_minus3 + 3;
			const uint32 CtbLog2SizeY = MinCbLog2SizeY + CurrentSPS.log2_diff_max_min_luma_coding_block_size;
			const uint32 CtbSizeY = 1u << CtbLog2SizeY;
			const uint32 PicWidthInCtbsY = FMath::DivideAndRoundUp<uint32>(CurrentSPS.pic_width_in_luma_samples, CtbSizeY);
			const uint32 PicHeightInCtbsY = FMath::DivideAndRoundUp<uint32>(CurrentSPS.pic_height_in_luma_samples, CtbSizeY);
			const uint32 PicSizeInCtbsY = PicWidthInCtbsY * PicHeightInCtbsY;

			Bitstream.ReadBits(OutSlice.slice_segment_address, FMath::CeilLogTwo(PicSizeInCtbsY)); // u(v)
        }

        uint32 CuQpDeltaVal = 0;
        if(!OutSlice.dependent_slice_segment_flag)
        {
            Bitstream.SkipBits(CurrentPPS.num_extra_slice_header_bits); // u(1) * num_extra_slice_header_bits
            Bitstream.Read(OutSlice.slice_type); // ue(v)

            if (CurrentPPS.output_flag_present_flag)
			{
				Bitstream.Read(OutSlice.pic_output_flag); // u(1)
			}

			if (CurrentSPS.separate_colour_plane_flag == 1)
			{
				Bitstream.Read(OutSlice.colour_plane_id); // u(2)
			}

            if (InNaluInfo.Type != ENaluType::IDR_W_RADL && InNaluInfo.Type != ENaluType::IDR_N_LP)
			{
				Bitstream.ReadBits(OutSlice.slice_pic_order_cnt_lsb, CurrentSPS.log2_max_pic_order_cnt_lsb_minus4 + 4); // u(v)
				Bitstream.Read(OutSlice.short_term_ref_pic_set_sps_flag); // u(1)

				if (!OutSlice.short_term_ref_pic_set_sps_flag)
				{
					CurrentSPS.short_term_ref_pic_sets[CurrentSPS.num_short_term_ref_pic_sets].Parse(CurrentSPS.num_short_term_ref_pic_sets, CurrentSPS.short_term_ref_pic_sets, Bitstream);
					CurrentSPS.short_term_ref_pic_sets[CurrentSPS.num_short_term_ref_pic_sets].CalculateValues(CurrentSPS.num_short_term_ref_pic_sets, CurrentSPS.short_term_ref_pic_sets);
				}
				else if (CurrentSPS.num_short_term_ref_pic_sets > 1)
				{
					Bitstream.ReadBits(OutSlice.short_term_ref_pic_set_idx, FMath::CeilLogTwo(CurrentSPS.num_short_term_ref_pic_sets)); // u(v)
				}

				if (CurrentSPS.long_term_ref_pics_present_flag)
				{
					if (CurrentSPS.num_long_term_ref_pics_sps > 0)
					{
						Bitstream.Read(OutSlice.num_long_term_sps); // ue(v)
					}

					Bitstream.Read(OutSlice.num_long_term_pics); // ue(v)

                    OutSlice.lt_idx_sps = new FNalu::U<>[OutSlice.num_long_term_sps];
                    for(uint16 i = 0; i < OutSlice.num_long_term_sps; i++)
                    {
                        OutSlice.lt_idx_sps[i] = 0;
                    }

                    OutSlice.poc_lsb_lt = new FNalu::U<>[OutSlice.num_long_term_sps + OutSlice.num_long_term_pics];
                    OutSlice.used_by_curr_pic_lt_flag = new FNalu::U<1>[OutSlice.num_long_term_sps + OutSlice.num_long_term_pics];
                    OutSlice.delta_poc_msb_present_flag = new FNalu::U<1>[OutSlice.num_long_term_sps + OutSlice.num_long_term_pics];
                    OutSlice.delta_poc_msb_cycle_lt = new FNalu::UE[OutSlice.num_long_term_sps + OutSlice.num_long_term_pics];
                    for(uint16 i = 0; i < (OutSlice.num_long_term_sps + OutSlice.num_long_term_pics); i++)
                    {
                        OutSlice.delta_poc_msb_cycle_lt[i] = 0;
                    }
					// OutSlice.lt_idx_sps.SetNumZeroed(OutSlice.num_long_term_sps);
					// OutSlice.poc_lsb_lt.SetNumZeroed(OutSlice.num_long_term_sps + OutSlice.num_long_term_pics);
					// OutSlice.used_by_curr_pic_lt_flag.SetNumZeroed(OutSlice.num_long_term_sps + OutSlice.num_long_term_pics);
					// OutSlice.delta_poc_msb_present_flag.SetNumZeroed(OutSlice.num_long_term_sps + OutSlice.num_long_term_pics);
					// OutSlice.delta_poc_msb_cycle_lt.SetNumZeroed(OutSlice.num_long_term_sps + OutSlice.num_long_term_pics);

					for (uint16 i = 0; i < OutSlice.num_long_term_sps + OutSlice.num_long_term_pics; i++)
					{
						if (i < OutSlice.num_long_term_sps)
						{
							if (OutSlice.num_long_term_sps > 1)
							{
								Bitstream.ReadBits(OutSlice.lt_idx_sps[i], FMath::CeilLogTwo(CurrentSPS.num_long_term_ref_pics_sps)); // u(v)
							}
						}
						else
						{
							Bitstream.ReadBits(OutSlice.poc_lsb_lt[i], CurrentSPS.log2_max_pic_order_cnt_lsb_minus4 + 4); // u(v)
							Bitstream.Read(OutSlice.used_by_curr_pic_lt_flag[i]);                                         // u(1)
						}

						Bitstream.Read(OutSlice.delta_poc_msb_present_flag[i]); // u(1)
						if (OutSlice.delta_poc_msb_present_flag[i])
						{
							Bitstream.Read(OutSlice.delta_poc_msb_cycle_lt[i]); // u(1)
						}
					}
				}

                if(CurrentSPS.sps_temporal_mvp_enabled_flag)
                {
                    Bitstream.Read(OutSlice.slice_temporal_mvp_enabled_flag); // u(1)
                }
			}  

            if (CurrentSPS.sample_adaptive_offset_enabled_flag)
			{
				Bitstream.Read(OutSlice.slice_sao_luma_flag); // u(1)
				if (CurrentSPS.ChromaArrayType != 0)
				{
					Bitstream.Read(OutSlice.slice_sao_chroma_flag); // u(1)
				}
			}

			if (OutSlice.slice_type == EH265SliceType::P || OutSlice.slice_type == EH265SliceType::B)
			{
                // Defaults
                OutSlice.num_ref_idx_l0_active_minus1 = CurrentPPS.num_ref_idx_l0_default_active_minus1;
                OutSlice.num_ref_idx_l1_active_minus1 = CurrentPPS.num_ref_idx_l1_default_active_minus1;
				Bitstream.Read(OutSlice.num_ref_idx_active_override_flag); // u(1)
				if (OutSlice.num_ref_idx_active_override_flag)
				{
					Bitstream.Read(OutSlice.num_ref_idx_l0_active_minus1); // ue(v)
					if (OutSlice.slice_type == EH265SliceType::B)
					{
						Bitstream.Read(OutSlice.num_ref_idx_l1_active_minus1); // ue(v)
					}
				}

                // Derive CurrRpsIdx (7.4.7.1)
			    OutSlice.CurrRpsIdx = CurrentSPS.num_short_term_ref_pic_sets;

			    if (OutSlice.short_term_ref_pic_set_sps_flag)
			    {
    				OutSlice.CurrRpsIdx = OutSlice.short_term_ref_pic_set_idx;
			    }

			    // Derive NumPicTotalCurr (7.4.7.2)
			    uint32 NumPicTotalCurr = 0;

			    for (uint32 i = 0; i < CurrentSPS.short_term_ref_pic_sets[OutSlice.CurrRpsIdx].NumNegativePics; i++)
			    {
    				if (CurrentSPS.short_term_ref_pic_sets[OutSlice.CurrRpsIdx].UsedByCurrPicS0[i])
				    {
    					NumPicTotalCurr++;
				    }
			    }

			    for (uint32 i = 0; i < CurrentSPS.short_term_ref_pic_sets[OutSlice.CurrRpsIdx].NumPositivePics; i++)
			    {
    				if (CurrentSPS.short_term_ref_pic_sets[OutSlice.CurrRpsIdx].UsedByCurrPicS1[i])
				    {
    					NumPicTotalCurr++;
				    }   
			    }

			    TArray<uint32> UsedByCurrPicLt;
			    UsedByCurrPicLt.SetNumZeroed(OutSlice.num_long_term_sps + OutSlice.num_long_term_pics);
    		    for (uint32 i = 0; i < OutSlice.num_long_term_sps + OutSlice.num_long_term_pics; i++)
	    		{
		    		if (i < OutSlice.num_long_term_sps)
			    	{
				    	UsedByCurrPicLt[i] = CurrentSPS.used_by_curr_pic_lt_sps_flag[OutSlice.lt_idx_sps[i]];
				    }
				    else
				    {
					    UsedByCurrPicLt[i] = OutSlice.used_by_curr_pic_lt_flag[i];
				    }

				    if (UsedByCurrPicLt[i])
    				{   
	    				NumPicTotalCurr++;
		    		}
    			}

			    if (CurrentPPS.pps_scc_extension.pps_curr_pic_ref_enabled_flag)
			    {
				    NumPicTotalCurr++;
			    }

			    // Derive PicOrderCntMsb (8.3.1)
			    uint32 MaxPicOrderCntLsb = 1u << (CurrentSPS.log2_max_pic_order_cnt_lsb_minus4 + 4);
			    uint8 PicOrderCntMsb;
			    if ((OutSlice.slice_pic_order_cnt_lsb < CurrentVPS.prevPicOrderCntLsb) && ((CurrentVPS.prevPicOrderCntLsb - OutSlice.slice_pic_order_cnt_lsb) >= (MaxPicOrderCntLsb / 2)))
			    {
    				PicOrderCntMsb = CurrentVPS.prevPicOrderCntMsb + MaxPicOrderCntLsb;
	    		}
    	    	else if ((OutSlice.slice_pic_order_cnt_lsb > CurrentVPS.prevPicOrderCntLsb) && ((OutSlice.slice_pic_order_cnt_lsb - CurrentVPS.prevPicOrderCntLsb) > (MaxPicOrderCntLsb / 2)))
			    {
				    PicOrderCntMsb = CurrentVPS.prevPicOrderCntMsb - MaxPicOrderCntLsb;
			    }
			    else
			    {
    				PicOrderCntMsb = CurrentVPS.prevPicOrderCntMsb;
			    }

			    OutSlice.CurrPicOrderCntVal = PicOrderCntMsb + OutSlice.slice_pic_order_cnt_lsb;

				if (CurrentPPS.lists_modification_present_flag && NumPicTotalCurr > 1)
    			{
					OutSlice.ref_pic_list_modification.Parse(OutSlice.num_ref_idx_l0_active_minus1, OutSlice.num_ref_idx_l1_active_minus1, NumPicTotalCurr, OutSlice.slice_type, Bitstream);
				}

				if (OutSlice.slice_type == EH265SliceType::B)
				{
					Bitstream.Read(OutSlice.mvd_l1_zero_flag); // u(1)
				}

				if (CurrentPPS.cabac_init_present_flag)
				{
					Bitstream.Read(OutSlice.cabac_init_flag); // u(1)
				}

				if (OutSlice.slice_temporal_mvp_enabled_flag)
				{
					if (OutSlice.slice_type == EH265SliceType::B)
					{
						Bitstream.Read(OutSlice.collocated_from_l0_flag); // u(1)
					}

					if ((OutSlice.collocated_from_l0_flag && OutSlice.num_ref_idx_l0_active_minus1 > 0) || (!OutSlice.collocated_from_l0_flag && OutSlice.num_ref_idx_l1_active_minus1 > 0))
					{
						Bitstream.Read(OutSlice.collocated_ref_idx); // ue(v)
					}
				}

				if ((CurrentPPS.weighted_pred_flag && OutSlice.slice_type == EH265SliceType::P) || (CurrentPPS.weighted_bipred_flag && OutSlice.slice_type == EH265SliceType::B))
				{
					OutSlice.pred_weight_table.Parse(CurrentSPS.ChromaArrayType, InNaluInfo, OutSlice, Bitstream);
				}

				Bitstream.Read(OutSlice.five_minus_max_num_merge_cand); // ue(v)

				if (CurrentSPS.sps_scc_extension.motion_vector_resolution_control_idc == 2)
				{
					Bitstream.Read(OutSlice.use_integer_mv_flag);	 // u(1)
				}
			}

			Bitstream.Read(OutSlice.slice_qp_delta); // se(v)


            // TODO (william.belcher)
            /*
			if (CurrentPPS.pps_slice_chroma_qp_offsets_present_flag)
			{
				Bitstream.Read(OutSlice.slice_cb_qp_offset,  // se(v)
					           OutSlice.slice_cr_qp_offset); // se(v)
			}

            if(CurrentPPS.pps_scc_extension.pps_slice_act_qp_offsets_present_flag)
            {
                Bitstream.Read(OutSlice.slice_act_y_qp_offset,   // se(v)
			                   OutSlice.slice_act_cb_qp_offset,  // se(v)
                               OutSlice.slice_act_cr_qp_offset); // se(v)
            }
			
            if (CurrentPPS.pps_range_extension.chroma_qp_offset_list_enabled_flag)
			{
				Bitstream.Read(OutSlice.cu_chroma_qp_offset_enabled_flag); // u(1)
			}

			if (CurrentPPS.deblocking_filter_override_enabled_flag)
			{
				Bitstream.Read(OutSlice.deblocking_filter_override_flag); // u(1)
			}

			if (OutSlice.deblocking_filter_override_flag)
			{
				Bitstream.Read(OutSlice.slice_deblocking_filter_disabled_flag); // u(1)
				if (!OutSlice.slice_deblocking_filter_disabled_flag)
				{
					Bitstream.Read(OutSlice.slice_beta_offset_div2, // se(v)
						           OutSlice.slice_tc_offset_div2);  // se(v)
				}
			}

			if (CurrentPPS.pps_loop_filter_across_slices_enabled_flag && (OutSlice.slice_sao_luma_flag || OutSlice.slice_sao_chroma_flag || !OutSlice.slice_deblocking_filter_disabled_flag))
			{
				Bitstream.Read(OutSlice.slice_loop_filter_across_slices_enabled_flag); // u(1)
			}  
            */
        }

        /*
        if (CurrentPPS.tiles_enabled_flag || CurrentPPS.entropy_coding_sync_enabled_flag)
		{
			Bitstream.Read(OutSlice.num_entry_point_offsets); // ue(v)
			if (OutSlice.num_entry_point_offsets > 0)
			{
				Bitstream.Read(OutSlice.offset_len_minus1); // ue(v)
				OutSlice.entry_point_offset_minus1.SetNumUninitialized(OutSlice.num_entry_point_offsets);

				for (uint8 i = 0; i < OutSlice.num_entry_point_offsets; i++)
				{
					Bitstream.ReadBits(OutSlice.entry_point_offset_minus1[i], OutSlice.offset_len_minus1 + 1); // u(v)
				}
			}
		}
        */

		if (CurrentPPS.slice_segment_header_extension_present_flag)
		{
			// TODO (aidan) handle slice extensions
		}
        
        return EAVResult::Success;
    }

    void Slice_t::ref_pic_list_modification_t::Parse(uint8 const& in_num_ref_idx_l0_active_minus1, uint8 const& in_num_ref_idx_l1_active_minus1, uint32 const& InNumPicTotalCurr, EH265SliceType const& SliceType, FBitstreamReader& Bitstream)
    {
        Bitstream.Read(ref_pic_list_modification_flag_l0);
        list_entry_l0 = new U<>[in_num_ref_idx_l0_active_minus1];
        for(uint8 i = 0; i < in_num_ref_idx_l0_active_minus1; i++)
        {
            list_entry_l0[i] = 0;
        }
		// list_entry_l0.SetNumZeroed(in_num_ref_idx_l0_active_minus1);
		if(ref_pic_list_modification_flag_l0)
		{
			for(uint8 i = 0; i < in_num_ref_idx_l0_active_minus1; i++)
			{
				Bitstream.ReadBits(list_entry_l0[i], FMath::CeilLogTwo(InNumPicTotalCurr));
			}
		}

		if (SliceType == EH265SliceType::B)
		{
			Bitstream.Read(ref_pic_list_modification_flag_l1);
            list_entry_l1 = new U<>[in_num_ref_idx_l1_active_minus1];
            for(uint8 i = 0; i < in_num_ref_idx_l0_active_minus1; i++)
            {
                list_entry_l1[i] = 0;
            }
			// list_entry_l1.SetNumZeroed(in_num_ref_idx_l1_active_minus1);
			if(ref_pic_list_modification_flag_l1)
			{
				for(uint8 i = 0; i < in_num_ref_idx_l1_active_minus1; i++)
				{
					Bitstream.ReadBits(list_entry_l1[i], FMath::CeilLogTwo(InNumPicTotalCurr));
				}
			}
		}
    }

    void Slice_t::pred_weight_table_t::Parse(const uint8& ChromaArrayType, FNaluH265 const& InNaluInfo, Slice_t const& CurrentSlice, FBitstreamReader& Bitstream)
	{
		// TODO (william.belcher): Add RefPicList to Slice_t. Needed to fix the highlighted lines
        /*
		Bitstream.Read(luma_log2_weight_denom);

		if (ChromaArrayType != 0)
		{
			Bitstream.Read(delta_chroma_log2_weight_denom);
		}

		for (uint8 i = 0; i <= CurrentSlice.num_ref_idx_l0_active_minus1; i++)
		{
            ******************************************************************************************************************************************************************************
			if ((CurrentSlice.RefPicList0[i].nuh_layer_id != InNaluInfo.NuhLayerId) || (CurrentSlice.RefPicList0[i].PicOrderCntVal != CurrentSlice.PicOrderCntVal))
            ******************************************************************************************************************************************************************************
			{
				Bitstream.Read(luma_weight_l0_flag[i]);
			}
		}

		if (ChromaArrayType != 0)
		{
			for (uint8 i = 0; i <= CurrentSlice.num_ref_idx_l0_active_minus1; i++)
			{
                ******************************************************************************************************************************************************************************
				if ((CurrentSlice.RefPicList0[i].nuh_layer_id != InNaluInfo.NuhLayerId) || (CurrentSlice.RefPicList0[i].PicOrderCntVal != CurrentSlice.PicOrderCntVal))
                ******************************************************************************************************************************************************************************
				{
					Bitstream.Read(chroma_weight_l0_flag[i]);
				}
			}
		}

		for (uint8 i = 0; i <= CurrentSlice.num_ref_idx_l0_active_minus1; i++)
		{
			if (luma_weight_l0_flag[i])
			{
				Bitstream.Read(delta_luma_weight_l0[i], luma_offset_l0[i]);
			}
			if (chroma_weight_l0_flag[i])
			{
				for (uint8 j = 0; j < 2; j++)
				{
					Bitstream.Read(delta_chroma_weight_l0[i][j], delta_chroma_offset_l0[i][j]);
				}
			}
		}

		if (CurrentSlice.slice_type == EH265SliceType::B)
		{
			for (uint8 i = 0; i <= CurrentSlice.num_ref_idx_l1_active_minus1; i++)
			{
				if ((CurrentSlice.RefPicList1[i].nuh_layer_id != InNaluInfo.NuhLayerId) || (CurrentSlice.RefPicList1[i].PicOrderCntVal != CurrentSlice.PicOrderCntVal))
				{
					Bitstream.Read(luma_weight_l1_flag[i]);
				}
			}

			if (ChromaArrayType != 0)
			{
				for (uint8 i = 0; i <= CurrentSlice.num_ref_idx_l1_active_minus1; i++)
				{
					if ((CurrentSlice.RefPicList1[i].nuh_layer_id != InNaluInfo.NuhLayerId) || (CurrentSlice.RefPicList1[i].PicOrderCntVal != CurrentSlice.PicOrderCntVal))
					{
						Bitstream.Read(chroma_weight_l1_flag[i]);
					}
				}
			}

			for (uint8 i = 0; i <= CurrentSlice.num_ref_idx_l1_active_minus1; i++)
			{
				if (luma_weight_l1_flag[i])
				{
					Bitstream.Read(delta_luma_weight_l1[i], luma_offset_l1[i]);
				}
				if (chroma_weight_l1_flag[i])
				{
					for (uint8 j = 0; j < 2; j++)
					{
						Bitstream.Read(delta_chroma_weight_l1[i][j], delta_chroma_offset_l1[i][j]);
					}
				}
			}
		}
        */
	}

    FAVResult ParseSEI(FBitstreamReader& Bitstream, FNaluH265 const& InNaluInfo, SEI_t& OutSEI)
    {
        return FAVResult(EAVResult::Success, TEXT("SEI Unimplemented"));
    }
} // namespace UE::AVCodecCore::H265
