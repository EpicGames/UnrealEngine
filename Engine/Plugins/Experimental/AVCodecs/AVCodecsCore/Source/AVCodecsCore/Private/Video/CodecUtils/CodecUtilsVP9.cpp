// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/CodecUtils/CodecUtilsVP9.h"

#include "Video/VideoEncoder.h"

namespace UE::AVCodecCore::VP9
{
	FAVResult ParseHeader(FBitstreamReader& Bitstream, Header_t& OutHeader)
	{
		Bitstream.Read(OutHeader.frame_marker); // u(2)
		if (OutHeader.frame_marker != 2)
		{
			return FAVResult(EAVResult::Error, TEXT("frame_marker != 2"), TEXT("VP9"));
		}

		Bitstream.Read(OutHeader.profile_low_bit, // u(1)
			OutHeader.profile_high_bit);		  // u(1)
		OutHeader.Profile = static_cast<EProfile>((OutHeader.profile_high_bit << 1) + OutHeader.profile_low_bit);

		if (OutHeader.Profile == EProfile::Profile3)
		{
			if (Bitstream.ReadBits(1) != 0)
			{
				// Reserved Zero
				return FAVResult(EAVResult::Error, TEXT("Reserved zero bit was not zero!"), TEXT("VP9"));
			}
		}

		Bitstream.Read(OutHeader.show_existing_frame); // u(1)
		if (OutHeader.show_existing_frame == 1)
		{
			Bitstream.Read(OutHeader.frame_to_show_map_idx); // u(3);
			return EAVResult::Success;
		}

		Bitstream.Read(OutHeader.frame_type, // u(1)
			OutHeader.show_frame,			 // u(1)
			OutHeader.error_resilient_mode); // u(1)

		if (OutHeader.frame_type == 0)
		{
			// Is Keyframe
			// frame sync code
			if (ParseFrameSyncCode(Bitstream, OutHeader) != EAVResult::Success)
			{
				return FAVResult(EAVResult::Error, TEXT("Failed to parse frame sync code"), TEXT("VP9"));
			}

			// color config.
			if (ParseColorConfig(Bitstream, OutHeader) != EAVResult::Success)
			{
				return FAVResult(EAVResult::Error, TEXT("Failed to parse color config"), TEXT("VP9"));
			}

			// frame size
			if (ParseFrameSize(Bitstream, OutHeader) != EAVResult::Success)
			{
				return FAVResult(EAVResult::Error, TEXT("Failed to parse frame size"), TEXT("VP9"));
			}
		}
		else
		{
			// Not keyframe
			if (OutHeader.show_frame == 0)
			{
				Bitstream.Read(OutHeader.intra_only); // u(1)
			}

			if (OutHeader.error_resilient_mode == 0)
			{
				Bitstream.Read(OutHeader.reset_frame_context); // u(2)
			}

			if (OutHeader.intra_only == 1)
			{
				if (ParseFrameSyncCode(Bitstream, OutHeader) != EAVResult::Success)
				{
					return FAVResult(EAVResult::Error, TEXT("Failed to parse frame sync code"), TEXT("VP9"));
				}

				if (((OutHeader.profile_high_bit << 1) + OutHeader.profile_low_bit) > 0)
				{
					// Color config.
					if (ParseColorConfig(Bitstream, OutHeader) != EAVResult::Success)
					{
						return FAVResult(EAVResult::Error, TEXT("Failed to parse color config"), TEXT("VP9"));
					}
				}
				else
				{
					OutHeader.color_space = EColorSpace::BT_601;
					OutHeader.sub_sampling = ESubSampling::k420;
				}
			}

			// TODO (william.belcher): More VP9 "P" and "B" frame values. Not needed at the moment
		}

		return EAVResult::Success;
	}

	FAVResult ParseFrameSyncCode(FBitstreamReader& Bitstream, Header_t& OutHeader)
	{
		Bitstream.Read(OutHeader.frame_sync_byte_0, // u(8)
			OutHeader.frame_sync_byte_1,			// u(8)
			OutHeader.frame_sync_byte_2);			// u(8)
		return ((OutHeader.frame_sync_byte_0 == 0x49) && (OutHeader.frame_sync_byte_1 == 0x83) && (OutHeader.frame_sync_byte_2 = 0x42)) ? EAVResult::Success : EAVResult::Error;
	}

	FAVResult ParseColorConfig(FBitstreamReader& Bitstream, Header_t& OutHeader)
	{
		if (OutHeader.Profile == EProfile::Profile2 || OutHeader.Profile == EProfile::Profile3)
		{
			Bitstream.Read(OutHeader.ten_or_twelve_bit);
			OutHeader.bit_depth = OutHeader.ten_or_twelve_bit ? EBitDepth::k12Bit : EBitDepth::k10Bit;
		}
		else
		{
			OutHeader.bit_depth = EBitDepth::k8Bit;
		}

		Bitstream.Read(OutHeader.color_space); // u(3)
		if (OutHeader.color_space != EColorSpace::RGB)
		{
			Bitstream.Read(OutHeader.color_range); // u(1)
			if (OutHeader.Profile == EProfile::Profile1 || OutHeader.Profile == EProfile::Profile3)
			{
				Bitstream.Read(OutHeader.sub_sampling); // u(2)
				if (Bitstream.ReadBits(1) != 0)
				{
					// Reserved Zero
					return EAVResult::Error;
				}
			}
			else
			{
				OutHeader.sub_sampling = ESubSampling::k420;
			}
		}
		else
		{
			OutHeader.color_range = EColorRange::Full;
			if (OutHeader.Profile == EProfile::Profile1 || OutHeader.Profile == EProfile::Profile3)
			{
				OutHeader.sub_sampling = ESubSampling::k444;
				if (Bitstream.ReadBits(1) != 0)
				{
					// Reserved Zero
					return EAVResult::Error;
				}
			}
		}
		return EAVResult::Success;
	}

	FAVResult ParseFrameSize(FBitstreamReader& Bitstream, Header_t& OutHeader)
	{
		Bitstream.Read(OutHeader.frame_width_minus_1, // u(16)
			OutHeader.frame_height_minus_1);		  // u(16)

		OutHeader.FrameWidth = OutHeader.frame_width_minus_1 + 1;
		OutHeader.FrameHeight = OutHeader.frame_height_minus_1 + 1;

		return EAVResult::Success;
	}

	EInterLayerPrediction ScalabilityModeToInterLayerPredMode(EScalabilityMode ScalabilityMode)
	{
		switch (ScalabilityMode)
		{
			case EScalabilityMode::None:
			case EScalabilityMode::L1T1:
			case EScalabilityMode::L1T2:
			case EScalabilityMode::L1T3:
			case EScalabilityMode::L2T1:
			case EScalabilityMode::L2T1h:
				return EInterLayerPrediction::On;
			case EScalabilityMode::L2T1_KEY:
				return EInterLayerPrediction::OnKeyPicture;
			case EScalabilityMode::L2T2:
			case EScalabilityMode::L2T2h:
				return EInterLayerPrediction::On;
			case EScalabilityMode::L2T2_KEY:
			case EScalabilityMode::L2T2_KEY_SHIFT:
				return EInterLayerPrediction::OnKeyPicture;
			case EScalabilityMode::L2T3:
			case EScalabilityMode::L2T3h:
				return EInterLayerPrediction::On;
			case EScalabilityMode::L2T3_KEY:
				return EInterLayerPrediction::OnKeyPicture;
			case EScalabilityMode::L3T1:
			case EScalabilityMode::L3T1h:
				return EInterLayerPrediction::On;
			case EScalabilityMode::L3T1_KEY:
				return EInterLayerPrediction::OnKeyPicture;
			case EScalabilityMode::L3T2:
			case EScalabilityMode::L3T2h:
				return EInterLayerPrediction::On;
			case EScalabilityMode::L3T2_KEY:
				return EInterLayerPrediction::OnKeyPicture;
			case EScalabilityMode::L3T3:
			case EScalabilityMode::L3T3h:
				return EInterLayerPrediction::On;
			case EScalabilityMode::L3T3_KEY:
				return EInterLayerPrediction::OnKeyPicture;
			case EScalabilityMode::S2T1:
			case EScalabilityMode::S2T1h:
			case EScalabilityMode::S2T2:
			case EScalabilityMode::S2T2h:
			case EScalabilityMode::S2T3:
			case EScalabilityMode::S2T3h:
			case EScalabilityMode::S3T1:
			case EScalabilityMode::S3T1h:
			case EScalabilityMode::S3T2:
			case EScalabilityMode::S3T2h:
			case EScalabilityMode::S3T3:
			case EScalabilityMode::S3T3h:
				return EInterLayerPrediction::Off;
		}

		// All scalability modes should be accounted for in the above switch
		unimplemented();
		return EInterLayerPrediction::Off;
	}

} // namespace UE::AVCodecCore::VP9
