// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/Configs/VideoDecoderConfigAV1.h"

#include "AVUtility.h"

REGISTER_TYPEID(FVideoDecoderConfigAV1);