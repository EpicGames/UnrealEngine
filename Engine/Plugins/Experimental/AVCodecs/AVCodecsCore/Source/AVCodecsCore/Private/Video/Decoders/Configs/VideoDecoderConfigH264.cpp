// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"

#include "AVUtility.h"

REGISTER_TYPEID(FVideoDecoderConfigH264);

FAVResult FVideoDecoderConfigH264::Parse(FVideoPacket const& Packet, TArray<UE::AVCodecCore::H264::Slice_t>& OutSlices)
{
	using namespace UE::AVCodecCore::H264;

	TArray<FNaluH264> FoundNalus;

	FAVResult Result;
	Result = FindNALUs(Packet, FoundNalus);

	if (Result.IsNotSuccess())
	{
		return Result;
	}

	for (auto& NaluInfo : FoundNalus)
	{		
		Result = EAVResult::Success;
		
		// NALUs are is usually an EBSP so we need to strip out the emulation prevention 3 byte making them RBSP
		TArray64<uint8> RBSP;
		RBSP.AddUninitialized(NaluInfo.Size);

		int32 RBSPsize = EBSPtoRBSP(RBSP.GetData(), NaluInfo.Data, NaluInfo.Size);
		FBitstreamReader BitStream(RBSP.GetData(), RBSPsize);

		switch (NaluInfo.Type)
		{
			case ENaluType::Unspecified:
			default:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Unspecified"), TEXT("H264"));
				break;
			case ENaluType::SliceOfNonIdrPicture:
				OutSlices.AddUninitialized();
				Result = ParseSliceHeader(BitStream, NaluInfo, SPS, PPS, OutSlices.Last());
				break;
			case ENaluType::SliceDataPartitionA:
				OutSlices.AddUninitialized();
				Result = ParseSliceHeader(BitStream, NaluInfo, SPS, PPS, OutSlices.Last());
				break;
			case ENaluType::SliceDataPartitionB:
				OutSlices.AddUninitialized();
				Result = ParseSliceHeader(BitStream, NaluInfo, SPS, PPS, OutSlices.Last());
				break;
			case ENaluType::SliceDataPartitionC:
				OutSlices.AddUninitialized();
				Result = ParseSliceHeader(BitStream, NaluInfo, SPS, PPS, OutSlices.Last());
				break;
			case ENaluType::SliceIdrPicture:
				OutSlices.AddUninitialized();
				Result = ParseSliceHeader(BitStream, NaluInfo, SPS, PPS, OutSlices.Last());
				break;
			case ENaluType::SupplementalEnhancementInformation:
				SEI.AddUninitialized();
				Result = ParseSEI(BitStream, NaluInfo, SEI.Last());
				break;
			case ENaluType::SequenceParameterSet:
				Result = ParseSPS(BitStream, NaluInfo, SPS);
				break;
			case ENaluType::PictureParameterSet:
				Result = ParsePPS(BitStream, NaluInfo, SPS, PPS);
				break;
			case ENaluType::AccessUnitDelimiter:
				break;
			case ENaluType::EndOfSequence:
				break;
			case ENaluType::EndOfStream:
				break;
			case ENaluType::FillerData:
				break;
			case ENaluType::SequenceParameterSetExtension:
				break;
			case ENaluType::PrefixNalUnit:
				break;
			case ENaluType::SubsetSequenceParameterSet:
				break;
			case ENaluType::Reserved16:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Reserved16"), TEXT("H264"));
				break;
			case ENaluType::Reserved17:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Reserved17"), TEXT("H264"));
				break;
			case ENaluType::Reserved18:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Reserved18"), TEXT("H264"));
				break;
			case ENaluType::SliceOfAnAuxiliaryCoded:
				break;
			case ENaluType::SliceExtension:
				break;
			case ENaluType::SliceExtensionForDepthView:
				break;
			case ENaluType::Reserved22:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Reserved22"), TEXT("H264"));
				break;
			case ENaluType::Reserved23:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Reserved23"), TEXT("H264"));
				break;
			case ENaluType::Unspecified24:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Unspecified24"), TEXT("H264"));
				break;
			case ENaluType::Unspecified25:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Unspecified25"), TEXT("H264"));
				break;
			case ENaluType::Unspecified26:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Unspecified26"), TEXT("H264"));
				break;
			case ENaluType::Unspecified27:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Unspecified27"), TEXT("H264"));
				break;
			case ENaluType::Unspecified28:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Unspecified28"), TEXT("H264"));
				break;
			case ENaluType::Unspecified29:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Unspecified29"), TEXT("H264"));
				break;
			case ENaluType::Unspecified30:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Unspecified30"), TEXT("H264"));
				break;
			case ENaluType::Unspecified31:
				Result = FAVResult(EAVResult::Error, TEXT("Unexpected ENaluType::Unspecified31"), TEXT("H264"));
				break;
		}

		if (Result.IsNotSuccess())
		{
			return Result;
		}
	}

	return Result;
}

TOptional<int> FVideoDecoderConfigH264::GetLastSliceQP(TArray<UE::AVCodecCore::H264::Slice_t>& Slices)
{
    using namespace UE::AVCodecCore::H264;

    Slice_t& LastSlice = Slices.Last();
    PPS_t& LastSlicePPS = PPS[LastSlice.pic_parameter_set_id];

    const int QP = 26 + LastSlicePPS.pic_init_qp_minus26 + LastSlice.slice_qp_delta;
    if(QP < 0 || QP > 51)
    {
		FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("QP of %d was outside of range 0 to 51 inclusive. Defaulting to 0"), QP), TEXT("H264"));
        return TOptional<int>();
    }
    return QP;
}
