// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/Configs/VideoDecoderConfigH265.h"

#include "AVUtility.h"
#include "Video/CodecUtils/CodecUtilsH265.h"

REGISTER_TYPEID(FVideoDecoderConfigH265);

FAVResult FVideoDecoderConfigH265::Parse(FVideoPacket const& Packet, TArray<UE::AVCodecCore::H265::Slice_t>& OutSlices)
{
	using namespace UE::AVCodecCore::H265;

	TArray<FNaluH265> FoundNalus;

	FAVResult Result;
	Result = FindNALUs(Packet, FoundNalus);

	if (Result.IsNotSuccess())
	{
		return Result;
	}

	for (auto& NaluInfo : FoundNalus)
	{
		Result = EAVResult::Success;

        TArray64<uint8> RBSP;
		RBSP.AddUninitialized(NaluInfo.Size);

		int32 RBSPsize = EBSPtoRBSP(RBSP.GetData(), NaluInfo.Data, NaluInfo.Size);
		FBitstreamReader BitStream(RBSP.GetData(), RBSPsize);

		switch (NaluInfo.Type)
		{
			case ENaluType::TRAIL_N:
			case ENaluType::TRAIL_R:
			case ENaluType::TSA_N:
			case ENaluType::TSA_R:
			case ENaluType::STSA_N:
			case ENaluType::STSA_R:
			case ENaluType::RADL_N:
			case ENaluType::RADL_R:
			case ENaluType::RASL_N:
			case ENaluType::RASL_R:
                OutSlices.AddUninitialized();
                Result = ParseSliceHeader(BitStream, NaluInfo, VPS, SPS, PPS, OutSlices.Last());
				break;
			case ENaluType::RSV_VCL_N10:
				checkNoEntry(); // Reserved
				break;
			case ENaluType::RSV_VCL_R11:
				checkNoEntry(); // Reserved
				break;
			case ENaluType::RSV_VCL_N12:
				checkNoEntry(); // Reserved
				break;
			case ENaluType::RSV_VCL_R13:
				checkNoEntry(); // Reserved
				break;
			case ENaluType::RSV_VCL_N14:
				checkNoEntry(); // Reserved
				break;
			case ENaluType::RSV_VCL_R15:
				checkNoEntry(); // Reserved
				break;
			case ENaluType::IDR_W_RADL:
			case ENaluType::IDR_N_LP:
				UE_LOG(LogTemp, Verbose, TEXT("H256 Parsing: Recieved IDR"));
			case ENaluType::BLA_W_LP:
			case ENaluType::BLA_W_RADL:
			case ENaluType::BLA_N_LP:
			case ENaluType::CRA_NUT:
				OutSlices.AddUninitialized();
                Result = ParseSliceHeader(BitStream, NaluInfo, VPS, SPS, PPS, OutSlices.Last());
				break;
			case ENaluType::RSV_IRAP_VCL22:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_IRAP_VCL23:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_VCL24:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_VCL25:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_VCL26:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_VCL27:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_VCL28:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_VCL29:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_VCL30:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_VCL31:
				checkNoEntry(); // Reserved
				break;
			case ENaluType::VPS_NUT:
                Result = ParseVPS(BitStream, NaluInfo, VPS);
				break;
			case ENaluType::SPS_NUT:
				Result = ParseSPS(BitStream, NaluInfo, VPS, SPS);
				break;
			case ENaluType::PPS_NUT:
				Result = ParsePPS(BitStream, NaluInfo, VPS, SPS, PPS);
			case ENaluType::AUD_NUT:
				break;
			case ENaluType::EOS_NUT:
				break;
			case ENaluType::EOB_NUT:
				break;
			case ENaluType::FD_NUT:
				break;
			case ENaluType::PREFIX_SEI_NUT:
			case ENaluType::SUFFIX_SEI_NUT:
				SEI.AddUninitialized();
				Result = ParseSEI(BitStream, NaluInfo, SEI.Last());
				break;
			case ENaluType::RSV_NVCL41:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_NVCL42:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_NVCL43:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_NVCL44:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_NVCL45:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_NVCL46:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::RSV_NVCL47:
				checkNoEntry(); // Reserved or Unspecified
				break;
			case ENaluType::UNSPECIFIED:
				checkNoEntry(); // Reserved or Unspecified
				break;
			default:
				checkNoEntry(); // Reserved or Unspecified
				break;
		}

		if (Result.IsNotSuccess())
		{
			return Result;
		}
	}

	return Result;
}

TOptional<int> FVideoDecoderConfigH265::GetLastSliceQP(TArray<UE::AVCodecCore::H265::Slice_t>& Slices)
{
    using namespace UE::AVCodecCore::H265;

    Slice_t& LastSlice = Slices.Last();
    PPS_t& LastSlicePPS = PPS[LastSlice.slice_pic_parameter_set_id];

    const int QP = 26 + LastSlicePPS.init_qp_minus26 + LastSlice.slice_qp_delta;
    if(QP < 0 || QP > 51)
    {
		FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("QP of %d was outside of range 0 to 51 inclusive. Defaulting to 0"), QP), TEXT("H265"));
        return TOptional<int>();
    }
    return QP;
}
