// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/Configs/VideoDecoderConfigVP9.h"

#include "AVUtility.h"

REGISTER_TYPEID(FVideoDecoderConfigVP9);

FAVResult FVideoDecoderConfigVP9::Parse(FVideoPacket const& Packet, UE::AVCodecCore::VP9::Header_t& OutHeader)
{
    using namespace UE::AVCodecCore::VP9;

    FBitstreamReader BitStream(Packet.DataPtr.Get(), Packet.DataSize);
    return ParseHeader(BitStream, OutHeader);
}