// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/DependencyDescriptor.h"

// Setters are named briefly to chain them when building the template.
FFrameDependencyTemplate& FFrameDependencyTemplate::SpatialLayerId(int SpatialLayer)
{
	SpatialId = SpatialLayer;
	return *this;
}

FFrameDependencyTemplate& FFrameDependencyTemplate::TemporalLayerId(int TemporalLayer)
{
	TemporalId = TemporalLayer;
	return *this;
}

FFrameDependencyTemplate& FFrameDependencyTemplate::Dtis(const FString& DTIS)
{
	DecodeTargetIndications.SetNum(DTIS.Len());

	for (int32 i = 0; i < DTIS.Len(); i++)
	{
		FString Symbol = DTIS.Mid(i, 1);

		EDecodeTargetIndication Indication;
		if (Symbol.Contains(TEXT("D")))
		{
			Indication = EDecodeTargetIndication::Discardable;
		}
		else if (Symbol.Contains(TEXT("R")))
		{
			Indication = EDecodeTargetIndication::Required;
		}
		else if (Symbol.Contains(TEXT("S")))
		{
			Indication = EDecodeTargetIndication::Switch;
		}
		else
		{
			// TODO (Migration): if(Symbol.Contains(TEXT("-"))) never works, so we'll just default to NotPresent
			Indication = EDecodeTargetIndication::NotPresent;
		}

		DecodeTargetIndications[i] = Indication;
	}

	return *this;
}

FFrameDependencyTemplate& FFrameDependencyTemplate::FrameDiff(std::initializer_list<int32> Diffs)
{
	FrameDiffs.Append(Diffs);
	return *this;
}

FFrameDependencyTemplate& FFrameDependencyTemplate::ChainDiff(std::initializer_list<int32> Diffs)
{
	ChainDiffs.Append(Diffs);
	return *this;
}
