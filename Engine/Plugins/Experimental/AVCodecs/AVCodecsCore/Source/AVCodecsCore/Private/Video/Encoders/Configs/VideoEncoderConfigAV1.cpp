// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/Configs/VideoEncoderConfigAV1.h"

#include "AVUtility.h"

REGISTER_TYPEID(FVideoEncoderConfigAV1);
