// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/Configs/VideoEncoderConfigVP9.h"

#include "AVUtility.h"

REGISTER_TYPEID(FVideoEncoderConfigVP9);
