// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/GenericFrameInfo.h"

FGenericFrameInfo::FGenericFrameInfo()
{
	ActiveDecodeTargets.Init(true, 32);
}