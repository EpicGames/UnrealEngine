// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/SVC/ScalabilityStructureFactory.h"

struct FNamedStructureFactory
{
	EScalabilityMode Name;
	// Use function pointer to make FNamedStructureFactory trivally destructable.
	TUniquePtr<FScalableVideoController> (*Factory)();
	FScalableVideoController::FStreamLayersConfig Config;
};

// Wrap MakeUnique function to have correct return type.
template <typename T>
TUniquePtr<FScalableVideoController> Create()
{
	return MakeUnique<T>();
}

template <typename T>
TUniquePtr<FScalableVideoController> CreateH()
{
	// 1.5:1 scaling, see https://w3c.github.io/webrtc-svc/#scalabilitymodes*
	typename FScalableVideoController::FIntFraction Factor;
	Factor.Num = 2;
	Factor.Den = 3;
	return MakeUnique<T>(Factor);
}

constexpr FScalableVideoController::FStreamLayersConfig ConfigL1T1 = {
	/*num_spatial_layers=*/1, /*num_temporal_layers=*/1,
	/*uses_reference_scaling=*/false
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL1T2 = {
	/*num_spatial_layers=*/1, /*num_temporal_layers=*/2,
	/*uses_reference_scaling=*/false
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL1T3 = {
	/*num_spatial_layers=*/1, /*num_temporal_layers=*/3,
	/*uses_reference_scaling=*/false
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL2T1 = {
	/*num_spatial_layers=*/2,
	/*num_temporal_layers=*/1,
	/*uses_reference_scaling=*/true,
	{ { 1, 2 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL2T1h = {
	/*num_spatial_layers=*/2,
	/*num_temporal_layers=*/1,
	/*uses_reference_scaling=*/true,
	{ { 2, 3 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL2T2 = {
	/*num_spatial_layers=*/2,
	/*num_temporal_layers=*/2,
	/*uses_reference_scaling=*/true,
	{ { 1, 2 }, { 1, 1 } } 
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL2T2h = {
	/*num_spatial_layers=*/2,
	/*num_temporal_layers=*/2,
	/*uses_reference_scaling=*/true,
	{ { 2, 3 }, { 1, 1 } } 
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL2T3 = {
	/*num_spatial_layers=*/2,
	/*num_temporal_layers=*/3,
	/*uses_reference_scaling=*/true,
	{ { 1, 2 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL2T3h = {
	/*num_spatial_layers=*/2,
	/*num_temporal_layers=*/3,
	/*uses_reference_scaling=*/true,
	{ { 2, 3 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL3T1 = {
	/*num_spatial_layers=*/3,
	/*num_temporal_layers=*/1,
	/*uses_reference_scaling=*/true,
	{ { 1, 4 }, { 1, 2 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL3T1h = {
	/*num_spatial_layers=*/3,
	/*num_temporal_layers=*/1,
	/*uses_reference_scaling=*/true,
	{ { 4, 9 }, { 2, 3 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL3T2 = {
	/*num_spatial_layers=*/3,
	/*num_temporal_layers=*/2,
	/*uses_reference_scaling=*/true,
	{ { 1, 4 }, { 1, 2 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL3T2h = {
	/*num_spatial_layers=*/3,
	/*num_temporal_layers=*/2,
	/*uses_reference_scaling=*/true,
	{ { 4, 9 }, { 2, 3 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL3T3 = {
	/*num_spatial_layers=*/3,
	/*num_temporal_layers=*/3,
	/*uses_reference_scaling=*/true,
	{ { 1, 4 }, { 1, 2 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigL3T3h = {
	/*num_spatial_layers=*/3,
	/*num_temporal_layers=*/3,
	/*uses_reference_scaling=*/true,
	{ { 4, 9 }, { 2, 3 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigS2T1 = {
	/*num_spatial_layers=*/2,
	/*num_temporal_layers=*/1,
	/*uses_reference_scaling=*/false,
	{ { 1, 2 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigS2T1h = {
	/*num_spatial_layers=*/2,
	/*num_temporal_layers=*/1,
	/*uses_reference_scaling=*/false,
	{ { 2, 3 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigS2T2 = {
	/*num_spatial_layers=*/2,
	/*num_temporal_layers=*/2,
	/*uses_reference_scaling=*/false,
	{ { 1, 2 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigS2T2h = {
	/*num_spatial_layers=*/2,
	/*num_temporal_layers=*/2,
	/*uses_reference_scaling=*/false,
	{ { 2, 3 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigS2T3 = {
	/*num_spatial_layers=*/2,
	/*num_temporal_layers=*/3,
	/*uses_reference_scaling=*/false,
	{ { 1, 2 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigS2T3h = {
	/*num_spatial_layers=*/2,
	/*num_temporal_layers=*/3,
	/*uses_reference_scaling=*/false,
	{ { 2, 3 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigS3T1 = {
	/*num_spatial_layers=*/3,
	/*num_temporal_layers=*/1,
	/*uses_reference_scaling=*/false,
	{ { 1, 4 }, { 1, 2 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigS3T1h = {
	/*num_spatial_layers=*/3,
	/*num_temporal_layers=*/1,
	/*uses_reference_scaling=*/false,
	{ { 4, 9 }, { 2, 3 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigS3T2 = {
	/*num_spatial_layers=*/3,
	/*num_temporal_layers=*/2,
	/*uses_reference_scaling=*/false,
	{ { 1, 4 }, { 1, 2 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigS3T2h = {
	/*num_spatial_layers=*/3,
	/*num_temporal_layers=*/2,
	/*uses_reference_scaling=*/false,
	{ { 4, 9 }, { 2, 3 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigS3T3 = {
	/*num_spatial_layers=*/3,
	/*num_temporal_layers=*/3,
	/*uses_reference_scaling=*/false,
	{ { 1, 4 }, { 1, 2 }, { 1, 1 } }
};

constexpr FScalableVideoController::FStreamLayersConfig ConfigS3T3h = {
	/*num_spatial_layers=*/3,
	/*num_temporal_layers=*/3,
	/*uses_reference_scaling=*/false,
	{ { 4, 9 }, { 2, 3 }, { 1, 1 } }
};

constexpr FNamedStructureFactory Factories[] = {
	{ EScalabilityMode::L1T1, Create<FScalableVideoControllerNoLayering>, ConfigL1T1 },
	{ EScalabilityMode::L1T2, Create<FScalabilityStructureL1T2>, ConfigL1T2 },
	{ EScalabilityMode::L1T3, Create<FScalabilityStructureL1T3>, ConfigL1T3 },
	{ EScalabilityMode::L2T1, Create<FScalabilityStructureL2T1>, ConfigL2T1 },
	{ EScalabilityMode::L2T1h, CreateH<FScalabilityStructureL2T1>, ConfigL2T1h },
	{ EScalabilityMode::L2T1_KEY, Create<FScalabilityStructureL2T1Key>, ConfigL2T1 },
	{ EScalabilityMode::L2T2, Create<FScalabilityStructureL2T2>, ConfigL2T2 },
	{ EScalabilityMode::L2T2h, CreateH<FScalabilityStructureL2T2>, ConfigL2T2h },
	{ EScalabilityMode::L2T2_KEY, Create<FScalabilityStructureL2T2Key>, ConfigL2T2 },
	{ EScalabilityMode::L2T2_KEY_SHIFT, Create<FScalabilityStructureL2T2KeyShift>, ConfigL2T2 },
	{ EScalabilityMode::L2T3, Create<FScalabilityStructureL2T3>, ConfigL2T3 },
	{ EScalabilityMode::L2T3h, CreateH<FScalabilityStructureL2T3>, ConfigL2T3h },
	{ EScalabilityMode::L2T3_KEY, Create<FScalabilityStructureL2T3Key>, ConfigL2T3 },
	{ EScalabilityMode::L3T1, Create<FScalabilityStructureL3T1>, ConfigL3T1 },
	{ EScalabilityMode::L3T1h, CreateH<FScalabilityStructureL3T1>, ConfigL3T1h },
	{ EScalabilityMode::L3T1_KEY, Create<FScalabilityStructureL3T1Key>, ConfigL3T1 },
	{ EScalabilityMode::L3T2, Create<FScalabilityStructureL3T2>, ConfigL3T2 },
	{ EScalabilityMode::L3T2h, CreateH<FScalabilityStructureL3T2>, ConfigL3T2h },
	{ EScalabilityMode::L3T2_KEY, Create<FScalabilityStructureL3T2Key>, ConfigL3T2 },
	{ EScalabilityMode::L3T3, Create<FScalabilityStructureL3T3>, ConfigL3T3 },
	{ EScalabilityMode::L3T3h, CreateH<FScalabilityStructureL3T3>, ConfigL3T3h },
	{ EScalabilityMode::L3T3_KEY, Create<FScalabilityStructureL3T3Key>, ConfigL3T3 },
	{ EScalabilityMode::S2T1, Create<FScalabilityStructureS2T1>, ConfigS2T1 },
	{ EScalabilityMode::S2T1h, CreateH<FScalabilityStructureS2T1>, ConfigS2T1h },
	{ EScalabilityMode::S2T2, Create<FScalabilityStructureS2T2>, ConfigS2T2 },
	{ EScalabilityMode::S2T2h, CreateH<FScalabilityStructureS2T2>, ConfigS2T2h },
	{ EScalabilityMode::S2T3, Create<FScalabilityStructureS2T3>, ConfigS2T3 },
	{ EScalabilityMode::S2T3h, CreateH<FScalabilityStructureS2T3>, ConfigS2T3h },
	{ EScalabilityMode::S3T1, Create<FScalabilityStructureS3T1>, ConfigS3T1 },
	{ EScalabilityMode::S3T1h, CreateH<FScalabilityStructureS3T1>, ConfigS3T1h },
	{ EScalabilityMode::S3T2, Create<FScalabilityStructureS3T2>, ConfigS3T2 },
	{ EScalabilityMode::S3T2h, CreateH<FScalabilityStructureS3T2>, ConfigS3T2h },
	{ EScalabilityMode::S3T3, Create<FScalabilityStructureS3T3>, ConfigS3T3 },
	{ EScalabilityMode::S3T3h, CreateH<FScalabilityStructureS3T3>, ConfigS3T3h },
};

TUniquePtr<FScalableVideoController> CreateScalabilityStructure(EScalabilityMode Name)
{
	for (const auto& Entry : Factories)
	{
		if (Entry.Name == Name)
		{
			return Entry.Factory();
		}
	}
	return nullptr;
}

TOptional<FScalableVideoController::FStreamLayersConfig> ScalabilityStructureConfig(EScalabilityMode Name)
{
	for (const auto& Entry : Factories)
	{
		if (Entry.Name == Name)
		{
			return Entry.Config;
		}
	}
	return FNullOpt(0);
}