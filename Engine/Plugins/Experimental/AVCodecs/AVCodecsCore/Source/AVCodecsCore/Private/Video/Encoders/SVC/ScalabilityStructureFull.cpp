// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/SVC/ScalabilityStructureFull.h"

#include "AVResult.h"

constexpr int FScalabilityStructureFullSvc::MaxNumSpatialLayers;
constexpr int FScalabilityStructureFullSvc::MaxNumTemporalLayers;
FString		  FScalabilityStructureFullSvc::FramePatternNames[6] = {
	  TEXT("None"),
	  TEXT("Key"),
	  TEXT("DeltaT2A"),
	  TEXT("DeltaT1"),
	  TEXT("DeltaT2B"),
	  TEXT("DeltaT0")
};

FScalabilityStructureFullSvc::FScalabilityStructureFullSvc(int NumSpatialLayers, int NumTemporalLayers, FIntFraction ResolutionFactor)
	: NumSpatialLayers(NumSpatialLayers)
	, NumTemporalLayers(NumTemporalLayers)
	, ResolutionFactor(ResolutionFactor)
	, CanReferenceT0FrameForSpatialId(false, MaxNumSpatialLayers)
	, CanReferenceT1FrameForSpatialId(false, MaxNumSpatialLayers)
{
	ActiveDecodeTargets.Init(false, 32);

	uint32_t Val = (uint32_t{ 1 } << (NumSpatialLayers * NumTemporalLayers)) - 1;

	for (uint8_t i = 0; i < 32; i++)
	{
		ActiveDecodeTargets[i] = (Val >> i) & 1;
	}
}

FScalabilityStructureFullSvc::FStreamLayersConfig FScalabilityStructureFullSvc::StreamConfig() const
{
	FStreamLayersConfig Result;
	Result.NumSpatialLayers = NumSpatialLayers;
	Result.NumTemporalLayers = NumTemporalLayers;
	Result.ScalingFactors[NumSpatialLayers - 1].Num = 1;
	Result.ScalingFactors[NumSpatialLayers - 1].Den = 1;
	for (int Sid = NumSpatialLayers - 1; Sid > 0; --Sid)
	{
		Result.ScalingFactors[Sid - 1].Num = ResolutionFactor.Num * Result.ScalingFactors[Sid].Num;
		Result.ScalingFactors[Sid - 1].Den = ResolutionFactor.Den * Result.ScalingFactors[Sid].Den;
	}
	Result.bUsesReferenceScaling = NumSpatialLayers > 1;

	return Result;
}

bool FScalabilityStructureFullSvc::TemporalLayerIsActive(int Tid) const
{
	if (Tid >= NumTemporalLayers)
	{
		return false;
	}

	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		if (DecodeTargetIsActive(Sid, Tid))
		{
			return true;
		}
	}

	return false;
}

EDecodeTargetIndication FScalabilityStructureFullSvc::Dti(int Sid, int Tid, const FScalableVideoController::FLayerFrameConfig& Config)
{
	if (Sid < Config.GetSpatialId() || Tid < Config.GetTemporalId())
	{
		return EDecodeTargetIndication::NotPresent;
	}

	if (Sid == Config.GetSpatialId())
	{
		if (Tid == 0)
		{
			return EDecodeTargetIndication::Switch;
		}
		if (Tid == Config.GetTemporalId())
		{
			return EDecodeTargetIndication::Discardable;
		}
		if (Tid > Config.GetTemporalId())
		{
			return EDecodeTargetIndication::Switch;
		}
	}

	if (Config.GetIsKeyframe() || Config.GetId() == EFramePattern::Key)
	{
		return EDecodeTargetIndication::Switch;
	}

	return EDecodeTargetIndication::Required;
}

FScalabilityStructureFullSvc::EFramePattern FScalabilityStructureFullSvc::NextPattern() const
{
	switch (LastPattern)
	{
		case EFramePattern::None:
			return EFramePattern::Key;
		case EFramePattern::DeltaT2B:
			return EFramePattern::DeltaT0;
		case EFramePattern::DeltaT2A:
			if (TemporalLayerIsActive(1))
			{
				return EFramePattern::DeltaT1;
			}
			return EFramePattern::DeltaT0;
		case EFramePattern::DeltaT1:
			if (TemporalLayerIsActive(2))
			{
				return EFramePattern::DeltaT2B;
			}
			return EFramePattern::DeltaT0;
		case EFramePattern::Key:
		case EFramePattern::DeltaT0:
			if (TemporalLayerIsActive(2))
			{
				return EFramePattern::DeltaT2A;
			}
			if (TemporalLayerIsActive(1))
			{
				return EFramePattern::DeltaT1;
			}
			return EFramePattern::DeltaT0;
	}

	unimplemented();
	return EFramePattern::None;
}

TArray<FScalableVideoController::FLayerFrameConfig> FScalabilityStructureFullSvc::NextFrameConfig(bool bRestart)
{
	TArray<FScalableVideoController::FLayerFrameConfig> Configs;
	if (ActiveDecodeTargets.Find(true) == INDEX_NONE)
	{
		LastPattern = EFramePattern::None;
		return Configs;
	}

	if (LastPattern == EFramePattern::None || bRestart)
	{
		CanReferenceT0FrameForSpatialId.Init(false, MaxNumSpatialLayers);
		LastPattern = EFramePattern::None;
	}

	EFramePattern CurrentPattern = NextPattern();

	TOptional<int> SpatialDependencyBufferId;
	switch (CurrentPattern)
	{
		case EFramePattern::DeltaT0:
		case EFramePattern::Key:
			// Disallow temporal references cross T0 on higher temporal layers.
			CanReferenceT1FrameForSpatialId.Init(false, MaxNumSpatialLayers);
			for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
			{
				if (!DecodeTargetIsActive(Sid, /*Tid=*/0))
				{
					// Next frame from the spatial layer `Sid` shouldn't depend on
					// potentially old previous frame from the spatial layer `Sid`.
					CanReferenceT0FrameForSpatialId[Sid] = false;
					continue;
				}

				FScalableVideoController::FLayerFrameConfig& Config = Configs.AddDefaulted_GetRef();
				Config.Id(CurrentPattern).SpatialLayerId(Sid).TemporalLayerId(0);

				if (SpatialDependencyBufferId)
				{
					Config.Reference(*SpatialDependencyBufferId);
				}
				else if (CurrentPattern == EFramePattern::Key)
				{
					Config.Keyframe();
				}

				if (CanReferenceT0FrameForSpatialId[Sid])
				{
					Config.ReferenceAndUpdate(BufferIndex(Sid, /*Tid=*/0));
				}
				else
				{
					// TODO(bugs.webrtc.org/11999): Propagate chain restart on delta frame
					// to ChainDiffCalculator
					Config.Update(BufferIndex(Sid, /*Tid=*/0));
				}

				SpatialDependencyBufferId = BufferIndex(Sid, /*Tid=*/0);
			}
			break;
		case EFramePattern::DeltaT1:
			for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
			{
				if (!DecodeTargetIsActive(Sid, /*Tid=*/1) || !CanReferenceT0FrameForSpatialId[Sid])
				{
					continue;
				}

				FScalableVideoController::FLayerFrameConfig& Config = Configs.AddDefaulted_GetRef();
				Config.Id(CurrentPattern).SpatialLayerId(Sid).TemporalLayerId(1);
				// Temporal reference.
				Config.Reference(BufferIndex(Sid, /*Tid=*/0));
				// Spatial reference unless this is the lowest active spatial layer.
				if (SpatialDependencyBufferId)
				{
					Config.Reference(*SpatialDependencyBufferId);
				}
				// No frame reference top layer frame, so no need save it into a buffer.
				if (NumTemporalLayers > 2 || Sid < NumSpatialLayers - 1)
				{
					Config.Update(BufferIndex(Sid, /*Tid=*/1));
				}
				SpatialDependencyBufferId = BufferIndex(Sid, /*Tid=*/1);
			}
			break;
		case EFramePattern::DeltaT2A:
		case EFramePattern::DeltaT2B:
			for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
			{
				if (!DecodeTargetIsActive(Sid, /*Tid=*/2) || !CanReferenceT0FrameForSpatialId[Sid])
				{
					continue;
				}

				FScalableVideoController::FLayerFrameConfig& Config = Configs.AddDefaulted_GetRef();
				Config.Id(CurrentPattern).SpatialLayerId(Sid).TemporalLayerId(2);
				// Temporal reference.
				if (CurrentPattern == EFramePattern::DeltaT2B && CanReferenceT1FrameForSpatialId[Sid])
				{
					Config.Reference(BufferIndex(Sid, /*Tid=*/1));
				}
				else
				{
					Config.Reference(BufferIndex(Sid, /*Tid=*/0));
				}
				// Spatial reference unless this is the lowest active spatial layer.
				if (SpatialDependencyBufferId)
				{
					Config.Reference(*SpatialDependencyBufferId);
				}
				// No frame reference top layer frame, so no need save it into a buffer.
				if (Sid < NumSpatialLayers - 1)
				{
					Config.Update(BufferIndex(Sid, /*Tid=*/2));
				}
				SpatialDependencyBufferId = BufferIndex(Sid, /*Tid=*/2);
			}
			break;
		case EFramePattern::None:
			checkNoEntry();
			break;
	}

	if (Configs.Num() == 0 && !bRestart)
	{
		FString Contents;
		Contents.Empty(ActiveDecodeTargets.Num());

		for (int32 Index = ActiveDecodeTargets.Num() - 1; Index >= 0; --Index)
		{
			Contents += ActiveDecodeTargets[Index] ? TEXT('1') : TEXT('0');
		}

		FAVResult::Log(EAVResult::Warning, FString::Printf(TEXT("Failed to generate configuration for L%dT%d with active decode targets %s and transition from %s to %s. Resetting"), NumSpatialLayers, NumTemporalLayers, *Contents.Mid(ActiveDecodeTargets.Num() - NumSpatialLayers * NumTemporalLayers), *FramePatternNames[LastPattern], *FramePatternNames[CurrentPattern]), TEXT("ScalabilityStructureFullSvc"));

		return NextFrameConfig(/*restart=*/true);
	}

	return Configs;
}

FGenericFrameInfo FScalabilityStructureFullSvc::OnEncodeDone(const FScalableVideoController::FLayerFrameConfig& Config)
{
	// When encoder drops all frames for a temporal unit, it is better to reuse
	// old temporal pattern rather than switch to next one, thus switch to next
	// pattern defered here from the `NextFrameConfig`.
	// In particular creating VP9 references rely on this behavior.
	LastPattern = static_cast<EFramePattern>(Config.GetId());
	if (Config.GetTemporalId() == 0)
	{
		CanReferenceT0FrameForSpatialId[Config.GetSpatialId()] = true;
	}
	if (Config.GetTemporalId() == 1)
	{
		CanReferenceT1FrameForSpatialId[Config.GetSpatialId()] = true;
	}

	FGenericFrameInfo FrameInfo;

	FrameInfo.SpatialId = Config.GetSpatialId();
	FrameInfo.TemporalId = Config.GetTemporalId();
	FrameInfo.EncoderBuffers = Config.GetBuffers();
	FrameInfo.DecodeTargetIndications.Reserve(NumSpatialLayers * NumTemporalLayers);
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		for (int Tid = 0; Tid < NumTemporalLayers; ++Tid)
		{
			FrameInfo.DecodeTargetIndications.Add(Dti(Sid, Tid, Config));
		}
	}

	if (Config.GetTemporalId() == 0)
	{
		FrameInfo.PartOfChain.SetNum(NumSpatialLayers);
		for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
		{
			FrameInfo.PartOfChain[Sid] = Config.GetSpatialId() <= Sid;
		}
	}
	else
	{
		FrameInfo.PartOfChain.Init(false, NumSpatialLayers);
	}

	FrameInfo.ActiveDecodeTargets = ActiveDecodeTargets;

	return FrameInfo;
}

void FScalabilityStructureFullSvc::OnRatesUpdated(const FVideoBitrateAllocation& Bitrates)
{
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		// Enable/disable spatial layers independetely.
		bool bActive = true;
		for (int Tid = 0; Tid < NumTemporalLayers; ++Tid)
		{
			// To enable temporal layer, require bitrates for lower temporal layers.
			bActive = bActive && Bitrates.GetBitrate(Sid, Tid) > 0;
			SetDecodeTargetIsActive(Sid, Tid, bActive);
		}
	}
}

FFrameDependencyStructure FScalabilityStructureL1T2::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 2;
	Structure.NumChains = 1;
	Structure.DecodeTargetProtectedByChain = { 0, 0 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(3);
	Templates[0].TemporalLayerId(0).Dtis(TEXT("SS")).ChainDiff({ 0 });
	Templates[1].TemporalLayerId(0).Dtis(TEXT("SS")).ChainDiff({ 2 }).FrameDiff({ 2 });
	Templates[2].TemporalLayerId(1).Dtis(TEXT("-D")).ChainDiff({ 1 }).FrameDiff({ 1 });
	return Structure;
}

FFrameDependencyStructure FScalabilityStructureL1T3::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 3;
	Structure.NumChains = 1;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 0 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(5);
	Templates[0].TemporalLayerId(0).Dtis(TEXT("SSS")).ChainDiff({ 0 });
	Templates[1].TemporalLayerId(0).Dtis(TEXT("SSS")).ChainDiff({ 4 }).FrameDiff({ 4 });
	Templates[2].TemporalLayerId(1).Dtis(TEXT("-DS")).ChainDiff({ 2 }).FrameDiff({ 2 });
	Templates[3].TemporalLayerId(2).Dtis(TEXT("-D")).ChainDiff({ 1 }).FrameDiff({ 1 });
	Templates[4].TemporalLayerId(2).Dtis(TEXT("--D")).ChainDiff({ 3 }).FrameDiff({ 1 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureL2T1::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 2;
	Structure.NumChains = 2;
	Structure.DecodeTargetProtectedByChain = { 0, 1 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(4);
	Templates[0].SpatialLayerId(0).Dtis(TEXT("SR")).ChainDiff({ 2, 1 }).FrameDiff({ 2 });
	Templates[1].SpatialLayerId(0).Dtis(TEXT("SS")).ChainDiff({ 0, 0 });
	Templates[2].SpatialLayerId(1).Dtis(TEXT("-S")).ChainDiff({ 1, 1 }).FrameDiff({ 2, 1 });
	Templates[3].SpatialLayerId(1).Dtis(TEXT("-S")).ChainDiff({ 1, 1 }).FrameDiff({ 1 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureL2T2::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 4;
	Structure.NumChains = 2;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 1, 1 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(6);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSSS")).ChainDiff({ 0, 0 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSRR")).ChainDiff({ 4, 3 }).FrameDiff({ 4 });
	Templates[2].SpatialLayerId(0).TemporalLayerId(1).Dtis(TEXT("-D-R")).ChainDiff({ 2, 1 }).FrameDiff({ 2 });
	Templates[3].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SS")).ChainDiff({ 1, 1 }).FrameDiff({ 1 });
	Templates[4].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SS")).ChainDiff({ 1, 1 }).FrameDiff({ 4, 1 });
	Templates[5].SpatialLayerId(1).TemporalLayerId(1).Dtis(TEXT("---D")).ChainDiff({ 3, 2 }).FrameDiff({ 2, 1 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureL2T3::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 6;
	Structure.NumChains = 2;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 0, 1, 1, 1 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(10);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSSRRR")).ChainDiff({ 8, 7 }).FrameDiff({ 8 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSSSSS")).ChainDiff({ 0, 0 });
	Templates[2].SpatialLayerId(0).TemporalLayerId(1).Dtis(TEXT("-DS-RR")).ChainDiff({ 4, 3 }).FrameDiff({ 4 });
	Templates[3].SpatialLayerId(0).TemporalLayerId(2).Dtis(TEXT("--D--R")).ChainDiff({ 2, 1 }).FrameDiff({ 2 });
	Templates[4].SpatialLayerId(0).TemporalLayerId(2).Dtis(TEXT("--D--R")).ChainDiff({ 6, 5 }).FrameDiff({ 2 });
	Templates[5].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("---SSS")).ChainDiff({ 1, 1 }).FrameDiff({ 8, 1 });
	Templates[6].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("---SSS")).ChainDiff({ 1, 1 }).FrameDiff({ 1 });
	Templates[7].SpatialLayerId(1).TemporalLayerId(1).Dtis(TEXT("----DS")).ChainDiff({ 5, 4 }).FrameDiff({ 4, 1 });
	Templates[8].SpatialLayerId(1).TemporalLayerId(2).Dtis(TEXT("-----D")).ChainDiff({ 3, 2 }).FrameDiff({ 2, 1 });
	Templates[9].SpatialLayerId(1).TemporalLayerId(2).Dtis(TEXT("-----D")).ChainDiff({ 7, 6 }).FrameDiff({ 2, 1 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureL3T1::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 3;
	Structure.NumChains = 3;
	Structure.DecodeTargetProtectedByChain = { 0, 1, 2 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(6);
	Templates[0].SpatialLayerId(0).Dtis(TEXT("SRR")).ChainDiff({ 3, 2, 1 }).FrameDiff({ 3 });
	Templates[1].SpatialLayerId(0).Dtis(TEXT("SSS")).ChainDiff({ 0, 0, 0 });
	Templates[2].SpatialLayerId(1).Dtis(TEXT("-SR")).ChainDiff({ 1, 1, 1 }).FrameDiff({ 3, 1 });
	Templates[3].SpatialLayerId(1).Dtis(TEXT("-SS")).ChainDiff({ 1, 1, 1 }).FrameDiff({ 1 });
	Templates[4].SpatialLayerId(2).Dtis(TEXT("--S")).ChainDiff({ 2, 1, 1 }).FrameDiff({ 3, 1 });
	Templates[5].SpatialLayerId(2).Dtis(TEXT("--S")).ChainDiff({ 2, 1, 1 }).FrameDiff({ 1 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureL3T2::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 6;
	Structure.NumChains = 3;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 1, 1, 2, 2 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(9);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSRRRR")).ChainDiff({ 6, 5, 4 }).FrameDiff({ 6 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSSSSS")).ChainDiff({ 0, 0, 0 });
	Templates[2].SpatialLayerId(0).TemporalLayerId(1).Dtis(TEXT("-D-R-R")).ChainDiff({ 3, 2, 1 }).FrameDiff({ 3 });
	Templates[3].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SSRR")).ChainDiff({ 1, 1, 1 }).FrameDiff({ 6, 1 });
	Templates[4].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SSSS")).ChainDiff({ 1, 1, 1 }).FrameDiff({ 1 });
	Templates[5].SpatialLayerId(1).TemporalLayerId(1).Dtis(TEXT("---D-R")).ChainDiff({ 4, 3, 2 }).FrameDiff({ 3, 1 });
	Templates[6].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("----SS")).ChainDiff({ 2, 1, 1 }).FrameDiff({ 6, 1 });
	Templates[7].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("----SS")).ChainDiff({ 2, 1, 1 }).FrameDiff({ 1 });
	Templates[8].SpatialLayerId(2).TemporalLayerId(1).Dtis(TEXT("-----D")).ChainDiff({ 5, 4, 3 }).FrameDiff({ 3, 1 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureL3T3::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 9;
	Structure.NumChains = 3;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 0, 1, 1, 1, 2, 2, 2 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(15);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSSRRRRRR")).ChainDiff({ 12, 11, 10 }).FrameDiff({ 12 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSSSSSSSS")).ChainDiff({ 0, 0, 0 });
	Templates[2].SpatialLayerId(0).TemporalLayerId(1).Dtis(TEXT("-DS-RR-RR")).ChainDiff({ 6, 5, 4 }).FrameDiff({ 6 });
	Templates[3].SpatialLayerId(0).TemporalLayerId(2).Dtis(TEXT("--D--R--R")).ChainDiff({ 3, 2, 1 }).FrameDiff({ 3 });
	Templates[4].SpatialLayerId(0).TemporalLayerId(2).Dtis(TEXT("--D--R--R")).ChainDiff({ 9, 8, 7 }).FrameDiff({ 3 });
	Templates[5].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("---SSSRRR")).ChainDiff({ 1, 1, 1 }).FrameDiff({ 12, 1 });
	Templates[6].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("---SSSSSS")).ChainDiff({ 1, 1, 1 }).FrameDiff({ 1 });
	Templates[7].SpatialLayerId(1).TemporalLayerId(1).Dtis(TEXT("----DS-RR")).ChainDiff({ 7, 6, 5 }).FrameDiff({ 6, 1 });
	Templates[8].SpatialLayerId(1).TemporalLayerId(2).Dtis(TEXT("-----D--R")).ChainDiff({ 4, 3, 2 }).FrameDiff({ 3, 1 });
	Templates[9].SpatialLayerId(1).TemporalLayerId(2).Dtis(TEXT("-----D--R")).ChainDiff({ 10, 9, 8 }).FrameDiff({ 3, 1 });
	Templates[10].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("------SSS")).ChainDiff({ 2, 1, 1 }).FrameDiff({ 12, 1 });
	Templates[11].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("------SSS")).ChainDiff({ 2, 1, 1 }).FrameDiff({ 1 });
	Templates[12].SpatialLayerId(2).TemporalLayerId(1).Dtis(TEXT("-------DS")).ChainDiff({ 8, 7, 6 }).FrameDiff({ 6, 1 });
	Templates[13].SpatialLayerId(2).TemporalLayerId(2).Dtis(TEXT("--------D")).ChainDiff({ 5, 4, 3 }).FrameDiff({ 3, 1 });
	Templates[14].SpatialLayerId(2).TemporalLayerId(2).Dtis(TEXT("--------D")).ChainDiff({ 11, 10, 9 }).FrameDiff({ 3, 1 });

	return Structure;
}