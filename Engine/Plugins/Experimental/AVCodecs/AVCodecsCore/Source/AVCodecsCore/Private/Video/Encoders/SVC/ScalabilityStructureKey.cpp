// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/SVC/ScalabilityStructureKey.h"

constexpr int FScalabilityStructureKeySvc::MaxNumSpatialLayers;
constexpr int FScalabilityStructureKeySvc::MaxNumTemporalLayers;

FScalabilityStructureKeySvc::FScalabilityStructureKeySvc(int NumSpatialLayers, int NumTemporalLayers)
	: NumSpatialLayers(NumSpatialLayers)
	, NumTemporalLayers(NumTemporalLayers)
	, SpatialIdIsEnabled(false, MaxNumSpatialLayers)
	, CanReferenceT1FrameForSpatialId(false, MaxNumSpatialLayers)
{
	ActiveDecodeTargets.Init(false, 32);

	uint32_t Val = (uint32_t{ 1 } << (NumSpatialLayers * NumTemporalLayers)) - 1;

	for (uint8_t i = 0; i < 32; i++)
	{
		ActiveDecodeTargets[i] = (Val >> i) & 1;
	}
}

FScalableVideoController::FStreamLayersConfig FScalabilityStructureKeySvc::StreamConfig() const
{
	FStreamLayersConfig Result;
	Result.NumSpatialLayers = NumSpatialLayers;
	Result.NumTemporalLayers = NumTemporalLayers;
	Result.ScalingFactors[NumSpatialLayers - 1] = { 1, 1 };
	for (int Sid = NumSpatialLayers - 1; Sid > 0; --Sid)
	{
		Result.ScalingFactors[Sid - 1] = { 1, 2 * Result.ScalingFactors[Sid].Den };
	}
	Result.bUsesReferenceScaling = true;
	return Result;
}

bool FScalabilityStructureKeySvc::TemporalLayerIsActive(int Tid) const
{
	if (Tid >= NumTemporalLayers)
	{
		return false;
	}
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		if (DecodeTargetIsActive(Sid, Tid))
		{
			return true;
		}
	}
	return false;
}

EDecodeTargetIndication FScalabilityStructureKeySvc::Dti(int Sid, int Tid, const FScalableVideoController::FLayerFrameConfig& Config)
{
	if (Config.GetIsKeyframe() || Config.GetId() == EFramePattern::Key)
	{
		return Sid < Config.GetSpatialId() ? EDecodeTargetIndication::NotPresent : EDecodeTargetIndication::Switch;
	}

	if (Sid != Config.GetSpatialId() || Tid < Config.GetTemporalId())
	{
		return EDecodeTargetIndication::NotPresent;
	}
	if (Tid == Config.GetTemporalId() && Tid > 0)
	{
		return EDecodeTargetIndication::Discardable;
	}
	return EDecodeTargetIndication::Switch;
}

TArray<FScalableVideoController::FLayerFrameConfig> FScalabilityStructureKeySvc::KeyframeConfig()
{
	TArray<FScalableVideoController::FLayerFrameConfig> Configs;
	TOptional<int>										SpatialDependencyBufferId;
	SpatialIdIsEnabled.Init(false, MaxNumSpatialLayers);
	// Disallow temporal references cross T0 on higher temporal layers.
	CanReferenceT1FrameForSpatialId.Init(false, MaxNumSpatialLayers);
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		if (!DecodeTargetIsActive(Sid, /*Tid=*/0))
		{
			continue;
		}

		FScalableVideoController::FLayerFrameConfig& Config = Configs.AddDefaulted_GetRef();
		Config.Id(EFramePattern::Key).SpatialLayerId(Sid).TemporalLayerId(0);

		if (SpatialDependencyBufferId)
		{
			Config.Reference(*SpatialDependencyBufferId);
		}
		else
		{
			Config.Keyframe();
		}
		Config.Update(BufferIndex(Sid, /*Tid=*/0));

		SpatialIdIsEnabled[Sid] = true;
		SpatialDependencyBufferId = BufferIndex(Sid, /*Tid=*/0);
	}
	return Configs;
}

TArray<FScalableVideoController::FLayerFrameConfig> FScalabilityStructureKeySvc::T0Config()
{
	TArray<FScalableVideoController::FLayerFrameConfig> Configs;
	// Disallow temporal references cross T0 on higher temporal layers.
	CanReferenceT1FrameForSpatialId.Init(false, MaxNumSpatialLayers);
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		if (!DecodeTargetIsActive(Sid, /*Tid=*/0))
		{
			SpatialIdIsEnabled[Sid] = false;
			continue;
		}
		FScalableVideoController::FLayerFrameConfig& Config = Configs.AddDefaulted_GetRef();
		Config.Id(EFramePattern::DeltaT0).SpatialLayerId(Sid).TemporalLayerId(0).ReferenceAndUpdate(BufferIndex(Sid, /*Tid=*/0));
	}
	return Configs;
}

TArray<FScalableVideoController::FLayerFrameConfig> FScalabilityStructureKeySvc::T1Config()
{
	TArray<FScalableVideoController::FLayerFrameConfig> Configs;
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		if (!DecodeTargetIsActive(Sid, /*Tid=*/1))
		{
			continue;
		}
		FScalableVideoController::FLayerFrameConfig& Config = Configs.AddDefaulted_GetRef();
		Config.Id(EFramePattern::DeltaT1).SpatialLayerId(Sid).TemporalLayerId(1).Reference(BufferIndex(Sid, /*Tid=*/0));
		if (NumTemporalLayers > 2)
		{
			Config.Update(BufferIndex(Sid, /*Tid=*/1));
		}
	}
	return Configs;
}

TArray<FScalableVideoController::FLayerFrameConfig> FScalabilityStructureKeySvc::T2Config(EFramePattern pattern)
{
	TArray<FScalableVideoController::FLayerFrameConfig> Configs;
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		if (!DecodeTargetIsActive(Sid, /*Tid=*/2))
		{
			continue;
		}
		FScalableVideoController::FLayerFrameConfig& Config = Configs.AddDefaulted_GetRef();
		Config.Id(pattern).SpatialLayerId(Sid).TemporalLayerId(2);
		if (CanReferenceT1FrameForSpatialId[Sid])
		{
			Config.Reference(BufferIndex(Sid, /*Tid=*/1));
		}
		else
		{
			Config.Reference(BufferIndex(Sid, /*Tid=*/0));
		}
	}
	return Configs;
}

FScalabilityStructureKeySvc::EFramePattern FScalabilityStructureKeySvc::NextPattern(EFramePattern InLastPattern) const
{
	switch (InLastPattern)
	{
		case EFramePattern::None:
			return EFramePattern::Key;
		case EFramePattern::DeltaT2B:
			return EFramePattern::DeltaT0;
		case EFramePattern::DeltaT2A:
			if (TemporalLayerIsActive(1))
			{
				return EFramePattern::DeltaT1;
			}
			return EFramePattern::DeltaT0;
		case EFramePattern::DeltaT1:
			if (TemporalLayerIsActive(2))
			{
				return EFramePattern::DeltaT2B;
			}
			return EFramePattern::DeltaT0;
		case EFramePattern::DeltaT0:
		case EFramePattern::Key:
			if (TemporalLayerIsActive(2))
			{
				return EFramePattern::DeltaT2A;
			}
			if (TemporalLayerIsActive(1))
			{
				return EFramePattern::DeltaT1;
			}
			return EFramePattern::DeltaT0;
	}

	unimplemented();
	return EFramePattern::None;
}

TArray<FScalableVideoController::FLayerFrameConfig> FScalabilityStructureKeySvc::NextFrameConfig(bool bRestart)
{
	if (ActiveDecodeTargets.Find(true) == INDEX_NONE)
	{
		LastPattern = EFramePattern::None;
		return {};
	}

	if (bRestart)
	{
		LastPattern = EFramePattern::None;
	}

	EFramePattern CurrentPattern = NextPattern(LastPattern);
	switch (CurrentPattern)
	{
		case EFramePattern::Key:
			return KeyframeConfig();
		case EFramePattern::DeltaT0:
			return T0Config();
		case EFramePattern::DeltaT1:
			return T1Config();
		case EFramePattern::DeltaT2A:
		case EFramePattern::DeltaT2B:
			return T2Config(CurrentPattern);
		case EFramePattern::None:
			break;
	}

	unimplemented();
	return {};
}

FGenericFrameInfo FScalabilityStructureKeySvc::OnEncodeDone(const FScalableVideoController::FLayerFrameConfig& Config)
{
	// When encoder drops all frames for a temporal unit, it is better to reuse
	// old temporal pattern rather than switch to next one, thus switch to next
	// pattern defered here from the `NextFrameConfig`.
	// In particular creating VP9 references rely on this behavior.
	LastPattern = static_cast<EFramePattern>(Config.GetId());
	if (Config.GetTemporalId() == 1)
	{
		CanReferenceT1FrameForSpatialId[Config.GetSpatialId()] = true;
	}

	FGenericFrameInfo FrameInfo;

	FrameInfo.SpatialId = Config.GetSpatialId();
	FrameInfo.TemporalId = Config.GetTemporalId();
	FrameInfo.EncoderBuffers = Config.GetBuffers();
	FrameInfo.DecodeTargetIndications.Reserve(NumSpatialLayers * NumTemporalLayers);
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		for (int Tid = 0; Tid < NumTemporalLayers; ++Tid)
		{
			FrameInfo.DecodeTargetIndications.Add(Dti(Sid, Tid, Config));
		}
	}

	FrameInfo.PartOfChain.Init(false, NumSpatialLayers);
	if (Config.GetIsKeyframe() || Config.GetId() == EFramePattern::Key)
	{
		for (int Sid = Config.GetSpatialId(); Sid < NumSpatialLayers; ++Sid)
		{
			FrameInfo.PartOfChain[Sid] = true;
		}
	}
	else if (Config.GetTemporalId() == 0)
	{
		FrameInfo.PartOfChain[Config.GetSpatialId()] = true;
	}

	FrameInfo.ActiveDecodeTargets = ActiveDecodeTargets;

	return FrameInfo;
}

void FScalabilityStructureKeySvc::OnRatesUpdated(const FVideoBitrateAllocation& Bitrates)
{
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		// Enable/disable spatial layers independetely.
		bool bActive = Bitrates.GetBitrate(Sid, /*Tid=*/0) > 0;
		SetDecodeTargetIsActive(Sid, /*Tid=*/0, bActive);
		if (!SpatialIdIsEnabled[Sid] && bActive)
		{
			// Key frame is required to reenable any spatial layer.
			LastPattern = EFramePattern::None;
		}

		for (int Tid = 1; Tid < NumTemporalLayers; ++Tid)
		{
			// To enable temporal layer, require Bitrates for lower temporal layers.
			bActive = bActive && Bitrates.GetBitrate(Sid, Tid) > 0;
			SetDecodeTargetIsActive(Sid, Tid, bActive);
		}
	}
}

FFrameDependencyStructure FScalabilityStructureL2T1Key::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 2;
	Structure.NumChains = 2;
	Structure.DecodeTargetProtectedByChain = { 0, 1 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(4);
	Templates[0].SpatialLayerId(0).Dtis(TEXT("S-")).ChainDiff({ 2, 1 }).FrameDiff({ 2 });
	Templates[1].SpatialLayerId(0).Dtis(TEXT("SS")).ChainDiff({ 0, 0 });
	Templates[2].SpatialLayerId(1).Dtis(TEXT("-S")).ChainDiff({ 1, 2 }).FrameDiff({ 2 });
	Templates[3].SpatialLayerId(1).Dtis(TEXT("-S")).ChainDiff({ 1, 1 }).FrameDiff({ 1 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureL2T2Key::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 4;
	Structure.NumChains = 2;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 1, 1 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(6);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSSS")).ChainDiff({ 0, 0 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SS--")).ChainDiff({ 4, 3 }).FrameDiff({ 4 });
	Templates[2].SpatialLayerId(0).TemporalLayerId(1).Dtis(TEXT("-D--")).ChainDiff({ 2, 1 }).FrameDiff({ 2 });
	Templates[3].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SS")).ChainDiff({ 1, 1 }).FrameDiff({ 1 });
	Templates[4].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SS")).ChainDiff({ 1, 4 }).FrameDiff({ 4 });
	Templates[5].SpatialLayerId(1).TemporalLayerId(1).Dtis(TEXT("---D")).ChainDiff({ 3, 2 }).FrameDiff({ 2 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureL2T3Key::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 6;
	Structure.NumChains = 2;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 0, 1, 1, 1 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(10);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSSSSS")).ChainDiff({ 0, 0 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSS---")).ChainDiff({ 8, 7 }).FrameDiff({ 8 });
	Templates[2].SpatialLayerId(0).TemporalLayerId(1).Dtis(TEXT("-DS---")).ChainDiff({ 4, 3 }).FrameDiff({ 4 });
	Templates[3].SpatialLayerId(0).TemporalLayerId(2).Dtis(TEXT("--D---")).ChainDiff({ 2, 1 }).FrameDiff({ 2 });
	Templates[4].SpatialLayerId(0).TemporalLayerId(2).Dtis(TEXT("--D---")).ChainDiff({ 6, 5 }).FrameDiff({ 2 });
	Templates[5].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("---SSS")).ChainDiff({ 1, 1 }).FrameDiff({ 1 });
	Templates[6].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("---SSS")).ChainDiff({ 1, 8 }).FrameDiff({ 8 });
	Templates[7].SpatialLayerId(1).TemporalLayerId(1).Dtis(TEXT("----DS")).ChainDiff({ 5, 4 }).FrameDiff({ 4 });
	Templates[8].SpatialLayerId(1).TemporalLayerId(2).Dtis(TEXT("-----D")).ChainDiff({ 3, 2 }).FrameDiff({ 2 });
	Templates[9].SpatialLayerId(1).TemporalLayerId(2).Dtis(TEXT("-----D")).ChainDiff({ 7, 6 }).FrameDiff({ 2 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureL3T1Key::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 3;
	Structure.NumChains = 3;
	Structure.DecodeTargetProtectedByChain = { 0, 1, 2 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(6);
	Templates[0].SpatialLayerId(0).Dtis(TEXT("S--")).ChainDiff({ 3, 2, 1 }).FrameDiff({ 3 });
	Templates[1].SpatialLayerId(0).Dtis(TEXT("SSS")).ChainDiff({ 0, 0, 0 });
	Templates[2].SpatialLayerId(1).Dtis(TEXT("-S-")).ChainDiff({ 1, 3, 2 }).FrameDiff({ 3 });
	Templates[3].SpatialLayerId(1).Dtis(TEXT("-SS")).ChainDiff({ 1, 1, 1 }).FrameDiff({ 1 });
	Templates[4].SpatialLayerId(2).Dtis(TEXT("--S")).ChainDiff({ 2, 1, 3 }).FrameDiff({ 3 });
	Templates[5].SpatialLayerId(2).Dtis(TEXT("--S")).ChainDiff({ 2, 1, 1 }).FrameDiff({ 1 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureL3T2Key::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 6;
	Structure.NumChains = 3;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 1, 1, 2, 2 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(9);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SS----")).ChainDiff({ 6, 5, 4 }).FrameDiff({ 6 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSSSSS")).ChainDiff({ 0, 0, 0 });
	Templates[2].SpatialLayerId(0).TemporalLayerId(1).Dtis(TEXT("-D----")).ChainDiff({ 3, 2, 1 }).FrameDiff({ 3 });
	Templates[3].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SS--")).ChainDiff({ 1, 6, 5 }).FrameDiff({ 6 });
	Templates[4].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SSSS")).ChainDiff({ 1, 1, 1 }).FrameDiff({ 1 });
	Templates[5].SpatialLayerId(1).TemporalLayerId(1).Dtis(TEXT("---D--")).ChainDiff({ 4, 3, 2 }).FrameDiff({ 3 });
	Templates[6].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("----SS")).ChainDiff({ 2, 1, 6 }).FrameDiff({ 6 });
	Templates[7].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("----SS")).ChainDiff({ 2, 1, 1 }).FrameDiff({ 1 });
	Templates[8].SpatialLayerId(2).TemporalLayerId(1).Dtis(TEXT("-----D")).ChainDiff({ 5, 4, 3 }).FrameDiff({ 3 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureL3T3Key::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 9;
	Structure.NumChains = 3;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 0, 1, 1, 1, 2, 2, 2 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(15);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSSSSSSSS")).ChainDiff({ 0, 0, 0 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSS------")).ChainDiff({ 12, 11, 10 }).FrameDiff({ 12 });
	Templates[2].SpatialLayerId(0).TemporalLayerId(1).Dtis(TEXT("-DS------")).ChainDiff({ 6, 5, 4 }).FrameDiff({ 6 });
	Templates[3].SpatialLayerId(0).TemporalLayerId(2).Dtis(TEXT("--D------")).ChainDiff({ 3, 2, 1 }).FrameDiff({ 3 });
	Templates[4].SpatialLayerId(0).TemporalLayerId(2).Dtis(TEXT("--D------")).ChainDiff({ 9, 8, 7 }).FrameDiff({ 3 });
	Templates[5].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("---SSSSSS")).ChainDiff({ 1, 1, 1 }).FrameDiff({ 1 });
	Templates[6].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("---SSS---")).ChainDiff({ 1, 12, 11 }).FrameDiff({ 12 });
	Templates[7].SpatialLayerId(1).TemporalLayerId(1).Dtis(TEXT("----DS---")).ChainDiff({ 7, 6, 5 }).FrameDiff({ 6 });
	Templates[8].SpatialLayerId(1).TemporalLayerId(2).Dtis(TEXT("-----D---")).ChainDiff({ 4, 3, 2 }).FrameDiff({ 3 });
	Templates[9].SpatialLayerId(1).TemporalLayerId(2).Dtis(TEXT("-----D---")).ChainDiff({ 10, 9, 8 }).FrameDiff({ 3 });
	Templates[10].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("------SSS")).ChainDiff({ 2, 1, 1 }).FrameDiff({ 1 });
	Templates[11].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("------SSS")).ChainDiff({ 2, 1, 12 }).FrameDiff({ 12 });
	Templates[12].SpatialLayerId(2).TemporalLayerId(1).Dtis(TEXT("-------DS")).ChainDiff({ 8, 7, 6 }).FrameDiff({ 6 });
	Templates[13].SpatialLayerId(2).TemporalLayerId(2).Dtis(TEXT("--------D")).ChainDiff({ 5, 4, 3 }).FrameDiff({ 3 });
	Templates[14].SpatialLayerId(2).TemporalLayerId(2).Dtis(TEXT("--------D")).ChainDiff({ 11, 10, 9 }).FrameDiff({ 3 });

	return Structure;
}