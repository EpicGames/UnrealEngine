// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/SVC/ScalabilityStructureL2T2KeyShift.h"

constexpr int FScalabilityStructureL2T2KeyShift::NumSpatialLayers;
constexpr int FScalabilityStructureL2T2KeyShift::NumTemporalLayers;

FScalabilityStructureL2T2KeyShift::FScalabilityStructureL2T2KeyShift()
{
	ActiveDecodeTargets.Init(false, 32);

	for (uint8_t i = 0; i < 3; i++)
	{
		ActiveDecodeTargets[i] = true;
	}
}

FScalableVideoController::FStreamLayersConfig FScalabilityStructureL2T2KeyShift::StreamConfig() const
{
	FStreamLayersConfig Result;
	Result.NumSpatialLayers = 2;
	Result.NumTemporalLayers = 2;
	Result.ScalingFactors[0] = { 1, 2 };
	Result.bUsesReferenceScaling = true;
	return Result;
}

FFrameDependencyStructure FScalabilityStructureL2T2KeyShift::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 4;
	Structure.NumChains = 2;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 1, 1 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(7);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSSS")).ChainDiff({ 0, 0 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SS--")).ChainDiff({ 2, 1 }).FrameDiff({ 2 });
	Templates[2].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SS--")).ChainDiff({ 4, 1 }).FrameDiff({ 4 });
	Templates[3].SpatialLayerId(0).TemporalLayerId(1).Dtis(TEXT("-D--")).ChainDiff({ 2, 3 }).FrameDiff({ 2 });
	Templates[4].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SS")).ChainDiff({ 1, 1 }).FrameDiff({ 1 });
	Templates[5].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SS")).ChainDiff({ 3, 4 }).FrameDiff({ 4 });
	Templates[6].SpatialLayerId(1).TemporalLayerId(1).Dtis(TEXT("---D")).ChainDiff({ 1, 2 }).FrameDiff({ 2 });

	return Structure;
}

TArray<FScalableVideoController::FLayerFrameConfig> FScalabilityStructureL2T2KeyShift::NextFrameConfig(bool bRestart)
{
	TArray<FScalableVideoController::FLayerFrameConfig> Configs;
	Configs.Reserve(2);
	if (bRestart)
	{
		NextPattern = EFramePattern::Key;
	}

	// Buffer0 keeps latest S0T0 frame,
	// Buffer1 keeps latest S1T0 frame.
	switch (NextPattern)
	{
		case EFramePattern::Key:
			if (DecodeTargetIsActive(/*Sid=*/0, /*Tid=*/0))
			{
				Configs.AddDefaulted_GetRef().SpatialLayerId(0).TemporalLayerId(0).Update(0).Keyframe();
			}
			if (DecodeTargetIsActive(/*Sid=*/1, /*Tid=*/0))
			{
				FScalableVideoController::FLayerFrameConfig& Config = Configs.AddDefaulted_GetRef();
				Config.SpatialLayerId(1).TemporalLayerId(0).Update(1);
				if (DecodeTargetIsActive(/*Sid=*/0, /*Tid=*/0))
				{
					Config.Reference(0);
				}
				else
				{
					Config.Keyframe();
				}
			}
			NextPattern = EFramePattern::Delta0;
			break;
		case EFramePattern::Delta0:
			if (DecodeTargetIsActive(/*Sid=*/0, /*Tid=*/0))
			{
				Configs.AddDefaulted_GetRef().SpatialLayerId(0).TemporalLayerId(0).ReferenceAndUpdate(0);
			}
			if (DecodeTargetIsActive(/*Sid=*/1, /*Tid=*/1))
			{
				Configs.AddDefaulted_GetRef().SpatialLayerId(1).TemporalLayerId(1).Reference(1);
			}
			if (Configs.Num() == 0 && DecodeTargetIsActive(/*Sid=*/1, /*Tid=*/0))
			{
				Configs.AddDefaulted_GetRef().SpatialLayerId(1).TemporalLayerId(0).ReferenceAndUpdate(1);
			}
			NextPattern = EFramePattern::Delta1;
			break;
		case EFramePattern::Delta1:
			if (DecodeTargetIsActive(/*Sid=*/0, /*Tid=*/1))
			{
				Configs.AddDefaulted_GetRef().SpatialLayerId(0).TemporalLayerId(1).Reference(0);
			}
			if (DecodeTargetIsActive(/*Sid=*/1, /*Tid=*/0))
			{
				Configs.AddDefaulted_GetRef().SpatialLayerId(1).TemporalLayerId(0).ReferenceAndUpdate(1);
			}
			if (Configs.Num() == 0 && DecodeTargetIsActive(/*Sid=*/0, /*Tid=*/0))
			{
				Configs.AddDefaulted_GetRef().SpatialLayerId(0).TemporalLayerId(0).ReferenceAndUpdate(0);
			}
			NextPattern = EFramePattern::Delta0;
			break;
	}

	return Configs;
}

FGenericFrameInfo FScalabilityStructureL2T2KeyShift::OnEncodeDone(const FScalableVideoController::FLayerFrameConfig& Config)
{
	FGenericFrameInfo FrameInfo;

	FrameInfo.SpatialId = Config.GetSpatialId();
	FrameInfo.TemporalId = Config.GetTemporalId();
	FrameInfo.EncoderBuffers = Config.GetBuffers();

	FrameInfo.DecodeTargetIndications.Reserve(NumSpatialLayers * NumTemporalLayers);
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		for (int Tid = 0; Tid < NumTemporalLayers; ++Tid)
		{
			FrameInfo.DecodeTargetIndications.Add(Dti(Sid, Tid, Config));
		}
	}

	if (Config.GetIsKeyframe())
	{
		FrameInfo.PartOfChain = { true, true };
	}
	else if (Config.GetTemporalId() == 0)
	{

		FrameInfo.PartOfChain = { Config.GetSpatialId() == 0, Config.GetSpatialId() == 1 };
	}
	else
	{
		FrameInfo.PartOfChain = { false, false };
	}

	return FrameInfo;
}

void FScalabilityStructureL2T2KeyShift::OnRatesUpdated(const FVideoBitrateAllocation& Bitrates)
{
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		// Enable/disable spatial layers independetely.
		bool bActive = Bitrates.GetBitrate(Sid, /*Tid=*/0) > 0;
		if (!DecodeTargetIsActive(Sid, /*Tid=*/0) && bActive)
		{
			// Key frame is required to reenable any spatial layer.
			NextPattern = EFramePattern::Key;
		}

		SetDecodeTargetIsActive(Sid, /*Tid=*/0, bActive);
		SetDecodeTargetIsActive(Sid, /*Tid=*/1, bActive && Bitrates.GetBitrate(Sid, /*Tid=*/1) > 0);
	}
}

EDecodeTargetIndication FScalabilityStructureL2T2KeyShift::Dti(int Sid, int Tid, const FScalableVideoController::FLayerFrameConfig& Config)
{
	if (Config.GetIsKeyframe())
	{
		return Sid < Config.GetSpatialId() ? EDecodeTargetIndication::NotPresent : EDecodeTargetIndication::Switch;
	}

	if (Sid != Config.GetSpatialId() || Tid < Config.GetTemporalId())
	{
		return EDecodeTargetIndication::NotPresent;
	}
	if (Tid == Config.GetTemporalId() && Tid > 0)
	{
		return EDecodeTargetIndication::Discardable;
	}
	return EDecodeTargetIndication::Switch;
}