// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/SVC/ScalabilityStructureSimulcast.h"

constexpr int FScalabilityStructureSimulcast::MaxNumSpatialLayers;
constexpr int FScalabilityStructureSimulcast::MaxNumTemporalLayers;

FScalabilityStructureSimulcast::FScalabilityStructureSimulcast(int NumSpatialLayers, int NumTemporalLayers, FIntFraction ResolutionFactor)
	: NumSpatialLayers(NumSpatialLayers)
	, NumTemporalLayers(NumTemporalLayers)
	, ResolutionFactor(ResolutionFactor)
	, CanReferenceT0FrameForSpatialId(false, MaxNumSpatialLayers)
	, CanReferenceT1FrameForSpatialId(false, MaxNumSpatialLayers)
{
	ActiveDecodeTargets.Init(false, 32);

	uint32_t Val = (uint32_t{ 1 } << (NumSpatialLayers * NumTemporalLayers)) - 1;

	for (uint8_t i = 0; i < 32; i++)
	{
		ActiveDecodeTargets[i] = (Val >> i) & 1;
	}
}

FScalableVideoController::FStreamLayersConfig FScalabilityStructureSimulcast::StreamConfig() const
{
	FStreamLayersConfig Result;
	Result.NumSpatialLayers = NumSpatialLayers;
	Result.NumTemporalLayers = NumTemporalLayers;
	Result.ScalingFactors[NumSpatialLayers - 1] = { 1, 1 };
	for (int Sid = NumSpatialLayers - 1; Sid > 0; --Sid)
	{
		Result.ScalingFactors[Sid - 1] = { ResolutionFactor.Num * Result.ScalingFactors[Sid].Num, ResolutionFactor.Den * Result.ScalingFactors[Sid].Den };
	}
	Result.bUsesReferenceScaling = false;
	return Result;
}

bool FScalabilityStructureSimulcast::TemporalLayerIsActive(int Tid) const
{
	if (Tid >= NumTemporalLayers)
	{
		return false;
	}
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		if (DecodeTargetIsActive(Sid, Tid))
		{
			return true;
		}
	}
	return false;
}

EDecodeTargetIndication FScalabilityStructureSimulcast::Dti(int Sid, int Tid, const FScalableVideoController::FLayerFrameConfig& Config)
{
	if (Sid != Config.GetSpatialId() || Tid < Config.GetTemporalId())
	{
		return EDecodeTargetIndication::NotPresent;
	}
	if (Tid == 0)
	{
		return EDecodeTargetIndication::Switch;
	}
	if (Tid == Config.GetTemporalId())
	{
		return EDecodeTargetIndication::Discardable;
	}

	return EDecodeTargetIndication::Switch;
}

FScalabilityStructureSimulcast::EFramePattern FScalabilityStructureSimulcast::NextPattern() const
{
	switch (LastPattern)
	{
		case EFramePattern::None:
		case EFramePattern::DeltaT2B:
			return EFramePattern::DeltaT0;
		case EFramePattern::DeltaT2A:
			if (TemporalLayerIsActive(1))
			{
				return EFramePattern::DeltaT1;
			}
			return EFramePattern::DeltaT0;
		case EFramePattern::DeltaT1:
			if (TemporalLayerIsActive(2))
			{
				return EFramePattern::DeltaT2B;
			}
			return EFramePattern::DeltaT0;
		case EFramePattern::DeltaT0:
			if (TemporalLayerIsActive(2))
			{
				return EFramePattern::DeltaT2A;
			}
			if (TemporalLayerIsActive(1))
			{
				return EFramePattern::DeltaT1;
			}
			return EFramePattern::DeltaT0;
	}

	unimplemented();
	return EFramePattern::DeltaT0;
}

TArray<FScalableVideoController::FLayerFrameConfig> FScalabilityStructureSimulcast::NextFrameConfig(bool bRestart)
{
	TArray<FScalableVideoController::FLayerFrameConfig> Configs;
	if (ActiveDecodeTargets.Find(true) == INDEX_NONE)
	{
		LastPattern = EFramePattern::None;
		return Configs;
	}
	Configs.Reserve(NumSpatialLayers);

	if (LastPattern == EFramePattern::None || bRestart)
	{
		CanReferenceT0FrameForSpatialId.Init(false, MaxNumSpatialLayers);
		LastPattern = EFramePattern::None;
	}
	EFramePattern CurrentPattern = NextPattern();

	switch (CurrentPattern)
	{
		case EFramePattern::DeltaT0:
			// Disallow temporal references cross T0 on higher temporal layers.
			CanReferenceT1FrameForSpatialId.Init(false, MaxNumSpatialLayers);
			for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
			{
				if (!DecodeTargetIsActive(Sid, /*Tid=*/0))
				{
					// Next frame from the spatial layer `Sid` shouldn't depend on
					// potentially old previous frame from the spatial layer `Sid`.
					CanReferenceT0FrameForSpatialId[Sid] = false;
					continue;
				}
				FScalableVideoController::FLayerFrameConfig& Config = Configs.AddDefaulted_GetRef();
				Config.Id(CurrentPattern).SpatialLayerId(Sid).TemporalLayerId(0);

				if (CanReferenceT0FrameForSpatialId[Sid])
				{
					Config.ReferenceAndUpdate(BufferIndex(Sid, /*Tid=*/0));
				}
				else
				{
					Config.Keyframe().Update(BufferIndex(Sid, /*Tid=*/0));
				}
				CanReferenceT0FrameForSpatialId[Sid] = true;
			}
			break;
		case EFramePattern::DeltaT1:
			for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
			{
				if (!DecodeTargetIsActive(Sid, /*Tid=*/1) || !CanReferenceT0FrameForSpatialId[Sid])
				{
					continue;
				}

				FScalableVideoController::FLayerFrameConfig& Config = Configs.AddDefaulted_GetRef();
				Config.Id(CurrentPattern).SpatialLayerId(Sid).TemporalLayerId(1).Reference(BufferIndex(Sid, /*Tid=*/0));
				// Save frame only if there is a higher temporal layer that may need it.
				if (NumTemporalLayers > 2)
				{
					Config.Update(BufferIndex(Sid, /*Tid=*/1));
				}
			}
			break;
		case EFramePattern::DeltaT2A:
		case EFramePattern::DeltaT2B:
			for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
			{
				if (!DecodeTargetIsActive(Sid, /*Tid=*/2) || !CanReferenceT0FrameForSpatialId[Sid])
				{
					continue;
				}
				FScalableVideoController::FLayerFrameConfig& Config = Configs.AddDefaulted_GetRef();
				Config.Id(CurrentPattern).SpatialLayerId(Sid).TemporalLayerId(2);
				if (CanReferenceT1FrameForSpatialId[Sid])
				{
					Config.Reference(BufferIndex(Sid, /*Tid=*/1));
				}
				else
				{
					Config.Reference(BufferIndex(Sid, /*Tid=*/0));
				}
			}
			break;
		case EFramePattern::None:
			checkNoEntry();
			break;
	}

	return Configs;
}

FGenericFrameInfo FScalabilityStructureSimulcast::OnEncodeDone(const FScalableVideoController::FLayerFrameConfig& Config)
{
	LastPattern = static_cast<EFramePattern>(Config.GetId());
	if (Config.GetTemporalId() == 1)
	{
		CanReferenceT1FrameForSpatialId[Config.GetSpatialId()] = true;
	}

	FGenericFrameInfo FrameInfo;

	FrameInfo.SpatialId = Config.GetSpatialId();
	FrameInfo.TemporalId = Config.GetTemporalId();
	FrameInfo.EncoderBuffers = Config.GetBuffers();

	FrameInfo.DecodeTargetIndications.Reserve(NumSpatialLayers * NumTemporalLayers);
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		for (int Tid = 0; Tid < NumTemporalLayers; ++Tid)
		{
			FrameInfo.DecodeTargetIndications.Add(Dti(Sid, Tid, Config));
		}
	}

	FrameInfo.PartOfChain.Init(false, NumSpatialLayers);

	if (Config.GetTemporalId() == 0)
	{
		FrameInfo.PartOfChain[Config.GetSpatialId()] = true;
	}
	FrameInfo.ActiveDecodeTargets = ActiveDecodeTargets;

	return FrameInfo;
}

void FScalabilityStructureSimulcast::OnRatesUpdated(const FVideoBitrateAllocation& Bitrates)
{
	for (int Sid = 0; Sid < NumSpatialLayers; ++Sid)
	{
		// Enable/disable spatial layers independetely.
		bool bActive = true;
		for (int Tid = 0; Tid < NumTemporalLayers; ++Tid)
		{
			// To enable temporal layer, require Bitrates for lower temporal layers.
			bActive = bActive && Bitrates.GetBitrate(Sid, Tid) > 0;
			SetDecodeTargetIsActive(Sid, Tid, bActive);
		}
	}
}

FFrameDependencyStructure FScalabilityStructureS2T1::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 2;
	Structure.NumChains = 2;
	Structure.DecodeTargetProtectedByChain = { 0, 1 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(4);
	Templates[0].SpatialLayerId(0).Dtis(TEXT("S-")).ChainDiff({ 2, 1 }).FrameDiff({ 2 });
	Templates[1].SpatialLayerId(0).Dtis(TEXT("S-")).ChainDiff({ 0, 0 });
	Templates[2].SpatialLayerId(1).Dtis(TEXT("-S")).ChainDiff({ 1, 2 }).FrameDiff({ 2 });
	Templates[3].SpatialLayerId(1).Dtis(TEXT("-S")).ChainDiff({ 1, 0 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureS2T2::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 4;
	Structure.NumChains = 2;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 1, 1 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(6);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SS--")).ChainDiff({ 4, 3 }).FrameDiff({ 4 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SS--")).ChainDiff({ 0, 0 });
	Templates[2].SpatialLayerId(0).TemporalLayerId(1).Dtis(TEXT("-D--")).ChainDiff({ 2, 1 }).FrameDiff({ 2 });
	Templates[3].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SS")).ChainDiff({ 1, 4 }).FrameDiff({ 4 });
	Templates[4].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SS")).ChainDiff({ 1, 0 });
	Templates[5].SpatialLayerId(1).TemporalLayerId(1).Dtis(TEXT("---D")).ChainDiff({ 3, 2 }).FrameDiff({ 2 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureS2T3::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 6;
	Structure.NumChains = 2;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 0, 1, 1, 1 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(10);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSS---")).ChainDiff({ 8, 7 }).FrameDiff({ 8 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSS---")).ChainDiff({ 0, 0 });
	Templates[2].SpatialLayerId(0).TemporalLayerId(1).Dtis(TEXT("-DS---")).ChainDiff({ 4, 3 }).FrameDiff({ 4 });
	Templates[3].SpatialLayerId(0).TemporalLayerId(2).Dtis(TEXT("--D---")).ChainDiff({ 2, 1 }).FrameDiff({ 2 });
	Templates[4].SpatialLayerId(0).TemporalLayerId(2).Dtis(TEXT("--D---")).ChainDiff({ 6, 5 }).FrameDiff({ 2 });
	Templates[5].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("---SSS")).ChainDiff({ 1, 8 }).FrameDiff({ 8 });
	Templates[6].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("---SSS")).ChainDiff({ 1, 0 });
	Templates[7].SpatialLayerId(1).TemporalLayerId(1).Dtis(TEXT("----DS")).ChainDiff({ 5, 4 }).FrameDiff({ 4 });
	Templates[8].SpatialLayerId(1).TemporalLayerId(2).Dtis(TEXT("-----D")).ChainDiff({ 3, 2 }).FrameDiff({ 2 });
	Templates[9].SpatialLayerId(1).TemporalLayerId(2).Dtis(TEXT("-----D")).ChainDiff({ 7, 6 }).FrameDiff({ 2 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureS3T1::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 3;
	Structure.NumChains = 3;
	Structure.DecodeTargetProtectedByChain = { 0, 1, 2 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(6);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("S--")).ChainDiff({ 3, 2, 1 }).FrameDiff({ 3 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("S--")).ChainDiff({ 0, 0, 0 });
	Templates[2].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("-S-")).ChainDiff({ 1, 3, 2 }).FrameDiff({ 3 });
	Templates[3].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("-S-")).ChainDiff({ 1, 0, 0 });
	Templates[4].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("--S")).ChainDiff({ 2, 1, 3 }).FrameDiff({ 3 });
	Templates[5].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("--S")).ChainDiff({ 2, 1, 0 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureS3T2::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 6;
	Structure.NumChains = 3;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 1, 1, 2, 2 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(9);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SS----")).ChainDiff({ 6, 5, 4 }).FrameDiff({ 6 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SS----")).ChainDiff({ 0, 0, 0 });
	Templates[2].SpatialLayerId(0).TemporalLayerId(1).Dtis(TEXT("-D----")).ChainDiff({ 3, 2, 1 }).FrameDiff({ 3 });
	Templates[3].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SS--")).ChainDiff({ 1, 6, 5 }).FrameDiff({ 6 });
	Templates[4].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("--SS--")).ChainDiff({ 1, 0, 0 });
	Templates[5].SpatialLayerId(1).TemporalLayerId(1).Dtis(TEXT("---D--")).ChainDiff({ 4, 3, 2 }).FrameDiff({ 3 });
	Templates[6].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("----SS")).ChainDiff({ 2, 1, 6 }).FrameDiff({ 6 });
	Templates[7].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("----SS")).ChainDiff({ 2, 1, 0 });
	Templates[8].SpatialLayerId(2).TemporalLayerId(1).Dtis(TEXT("-----D")).ChainDiff({ 5, 4, 3 }).FrameDiff({ 3 });

	return Structure;
}

FFrameDependencyStructure FScalabilityStructureS3T3::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 9;
	Structure.NumChains = 3;
	Structure.DecodeTargetProtectedByChain = { 0, 0, 0, 1, 1, 1, 2, 2, 2 };
	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(15);
	Templates[0].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSS------")).ChainDiff({ 12, 11, 10 }).FrameDiff({ 12 });
	Templates[1].SpatialLayerId(0).TemporalLayerId(0).Dtis(TEXT("SSS------")).ChainDiff({ 0, 0, 0 });
	Templates[2].SpatialLayerId(0).TemporalLayerId(1).Dtis(TEXT("-DS------")).ChainDiff({ 6, 5, 4 }).FrameDiff({ 6 });
	Templates[3].SpatialLayerId(0).TemporalLayerId(2).Dtis(TEXT("--D------")).ChainDiff({ 3, 2, 1 }).FrameDiff({ 3 });
	Templates[4].SpatialLayerId(0).TemporalLayerId(2).Dtis(TEXT("--D------")).ChainDiff({ 9, 8, 7 }).FrameDiff({ 3 });
	Templates[5].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("---SSS---")).ChainDiff({ 1, 12, 11 }).FrameDiff({ 12 });
	Templates[6].SpatialLayerId(1).TemporalLayerId(0).Dtis(TEXT("---SSS---")).ChainDiff({ 1, 0, 0 });
	Templates[7].SpatialLayerId(1).TemporalLayerId(1).Dtis(TEXT("----DS---")).ChainDiff({ 7, 6, 5 }).FrameDiff({ 6 });
	Templates[8].SpatialLayerId(1).TemporalLayerId(2).Dtis(TEXT("-----D---")).ChainDiff({ 4, 3, 2 }).FrameDiff({ 3 });
	Templates[9].SpatialLayerId(1).TemporalLayerId(2).Dtis(TEXT("-----D---")).ChainDiff({ 10, 9, 8 }).FrameDiff({ 3 });
	Templates[10].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("------SSS")).ChainDiff({ 2, 1, 12 }).FrameDiff({ 12 });
	Templates[11].SpatialLayerId(2).TemporalLayerId(0).Dtis(TEXT("------SSS")).ChainDiff({ 2, 1, 0 });
	Templates[12].SpatialLayerId(2).TemporalLayerId(1).Dtis(TEXT("-------DS")).ChainDiff({ 8, 7, 6 }).FrameDiff({ 6 });
	Templates[13].SpatialLayerId(2).TemporalLayerId(2).Dtis(TEXT("--------D")).ChainDiff({ 5, 4, 3 }).FrameDiff({ 3 });
	Templates[14].SpatialLayerId(2).TemporalLayerId(2).Dtis(TEXT("--------D")).ChainDiff({ 11, 10, 9 }).FrameDiff({ 3 });

	return Structure;
}