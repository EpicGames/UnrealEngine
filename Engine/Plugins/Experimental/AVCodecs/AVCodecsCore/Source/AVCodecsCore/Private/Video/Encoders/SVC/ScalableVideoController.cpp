// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/SVC/ScalableVideoController.h"

FScalableVideoController::FLayerFrameConfig& FScalableVideoController::FLayerFrameConfig::Id(int Value)
{
	LayerId = Value;
	return *this;
}

FScalableVideoController::FLayerFrameConfig& FScalableVideoController::FLayerFrameConfig::Keyframe()
{
	bIsKeyFrame = true;
	return *this;
}

FScalableVideoController::FLayerFrameConfig& FScalableVideoController::FLayerFrameConfig::SpatialLayerId(int Value)
{
	SpatialId = Value;
	return *this;
}

FScalableVideoController::FLayerFrameConfig& FScalableVideoController::FLayerFrameConfig::TemporalLayerId(int Value)
{
	TemporalId = Value;
	return *this;
}

FScalableVideoController::FLayerFrameConfig& FScalableVideoController::FLayerFrameConfig::Reference(int BufferId)
{
	Buffers.Add({ BufferId, /*referenced=*/true, /*updated=*/false });
	return *this;
}

FScalableVideoController::FLayerFrameConfig& FScalableVideoController::FLayerFrameConfig::Update(int BufferId)
{
	Buffers.Add({ BufferId, /*referenced=*/false, /*updated=*/true });
	return *this;
}

FScalableVideoController::FLayerFrameConfig& FScalableVideoController::FLayerFrameConfig::ReferenceAndUpdate(int BufferId)
{
	Buffers.Add(FCodecBufferUsage{ BufferId, /*referenced=*/true, /*updated=*/true });
	return *this;
}

int FScalableVideoController::FLayerFrameConfig::GetId() const
{
	return LayerId;
}

bool FScalableVideoController::FLayerFrameConfig::GetIsKeyframe() const
{
	return bIsKeyFrame;
}

int FScalableVideoController::FLayerFrameConfig::GetSpatialId() const
{
	return SpatialId;
}

int FScalableVideoController::FLayerFrameConfig::GetTemporalId() const
{
	return TemporalId;
}

const TArray<FCodecBufferUsage>& FScalableVideoController::FLayerFrameConfig::GetBuffers() const
{
	return Buffers;
}