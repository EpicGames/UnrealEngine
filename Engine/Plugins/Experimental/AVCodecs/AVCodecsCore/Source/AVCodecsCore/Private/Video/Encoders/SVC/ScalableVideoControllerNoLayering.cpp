// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/SVC/ScalableVideoControllerNoLayering.h"

FScalableVideoController::FStreamLayersConfig FScalableVideoControllerNoLayering::StreamConfig() const
{
	FScalableVideoController::FStreamLayersConfig Result;
	Result.NumSpatialLayers = 1;
	Result.NumTemporalLayers = 1;
	Result.bUsesReferenceScaling = false;
	return Result;
}

FFrameDependencyStructure FScalableVideoControllerNoLayering::DependencyStructure() const
{
	FFrameDependencyStructure Structure;
	Structure.NumDecodeTargets = 1;
	Structure.NumChains = 1;
	Structure.DecodeTargetProtectedByChain = { 0 };

	TArray<FFrameDependencyTemplate>& Templates = Structure.Templates;
	Templates.SetNum(2);
	Templates[0].Dtis(TEXT("S")).ChainDiff({ 0 });
	Templates[1].Dtis(TEXT("S")).ChainDiff({ 1 }).FrameDiff({ 1 });

	return Structure;
}

TArray<FScalableVideoController::FLayerFrameConfig> FScalableVideoControllerNoLayering::NextFrameConfig(bool bRestart)
{
	if (!bEnabled)
	{
		return {};
	}

	TArray<FScalableVideoController::FLayerFrameConfig> Result;
	Result.AddDefaulted(1);
	if (bRestart || bStart)
	{
		Result[0].Id(0).Keyframe().Update(0);
	}
	else
	{
		Result[0].Id(0).ReferenceAndUpdate(0);
	}
	bStart = false;
	return Result;
}

FGenericFrameInfo FScalableVideoControllerNoLayering::OnEncodeDone(const FScalableVideoController::FLayerFrameConfig& Config)
{
	FGenericFrameInfo FrameInfo;

	FrameInfo.EncoderBuffers = Config.GetBuffers();
	if (Config.GetIsKeyframe())
	{
		for (uint64_t i = 0; i < FrameInfo.EncoderBuffers.Num(); i++)
		{
			FrameInfo.EncoderBuffers[i].bReferenced = false;
		}
	}
	FrameInfo.DecodeTargetIndications = { EDecodeTargetIndication::Switch };
	FrameInfo.PartOfChain = { true };

	return FrameInfo;
}

void FScalableVideoControllerNoLayering::OnRatesUpdated(const FVideoBitrateAllocation& Bitrates)
{
	bEnabled = Bitrates.GetBitrate(0, 0) > 0;
}