// Copyright Epic Games, Inc. All Rights Reserved.
#include "Video/Encoders/SVC/VideoBitrateAllocatorSVC.h"

#include "Algo/Accumulate.h"
#include "Video/Encoders/SVC/ScalabilityStructureFactory.h"
#include "Video/Encoders/SVC/ScalableVideoController.h"

constexpr float SpatialLayeringRateScalingFactor = 0.55f;
constexpr float TemporalLayeringRateScalingFactor = 0.55f;

struct FActiveSpatialLayers
{
	size_t First = 0;
	size_t Num = 0;
};

FActiveSpatialLayers GetActiveSpatialLayers(const FVideoEncoderConfig &Config, size_t NumSpatialLayers)
{
	FActiveSpatialLayers Active;
	for (Active.First = 0; Active.First < NumSpatialLayers; ++Active.First)
	{
		if (Config.SpatialLayers[Active.First].bActive)
		{
			break;
		}
	}

	size_t LastActiveLayer = Active.First;
	for (; LastActiveLayer < NumSpatialLayers; ++LastActiveLayer)
	{
		if (!Config.SpatialLayers[LastActiveLayer].bActive)
		{
			break;
		}
	}
	Active.Num = LastActiveLayer - Active.First;

	return Active;
}

TArray<uint32> AdjustAndVerify(const FVideoEncoderConfig &Config, size_t FirstActiveLayer, const TArray<uint32> &SpatialLayerRates)
{
	TArray<uint32> AdjustedSpatialLayerRates;
	// Keep track of rate that couldn't be applied to the previous layer due to
	// max bitrate constraint, try to pass it forward to the next one.
	uint32 ExcessRate = 0.f;
	for (size_t sl_idx = 0; sl_idx < SpatialLayerRates.Num(); ++sl_idx)
	{
		uint32 MinRate = Config.SpatialLayers[FirstActiveLayer + sl_idx].MinBitrate * 1000;
		uint32 MaxRate = Config.SpatialLayers[FirstActiveLayer + sl_idx].MaxBitrate * 1000;

		uint32 LayerRate = SpatialLayerRates[sl_idx] + ExcessRate;
		if (LayerRate < MinRate)
		{
			// Not enough rate to reach min bitrate for desired number of Layers,
			// abort Allocation.
			if (SpatialLayerRates.Num() == 1)
			{
				return SpatialLayerRates;
			}
			return AdjustedSpatialLayerRates;
		}

		if (LayerRate <= MaxRate)
		{
			ExcessRate = 0.f;
			AdjustedSpatialLayerRates.Add(LayerRate);
		}
		else
		{
			ExcessRate = LayerRate - MaxRate;
			AdjustedSpatialLayerRates.Add(MaxRate);
		}
	}

	return AdjustedSpatialLayerRates;
}

static TArray<uint32> SplitBitrate(size_t NumLayers, uint32 TotalBitrateBps, float RateScalingFactor)
{
	TArray<uint32> Bitrates;

	double Denominator = 0.0;
	for (size_t LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
	{
		Denominator += FMath::Pow(RateScalingFactor, LayerIdx);
	}

	double Numerator = FMath::Pow(RateScalingFactor, NumLayers - 1);
	for (size_t LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
	{
		Bitrates.Add(Numerator * TotalBitrateBps / Denominator);
		Numerator /= RateScalingFactor;
	}

	const uint32 Sum = Algo::Accumulate(Bitrates, 0.f);

	// Keep the Sum of split Bitrates equal to the total bitrate by adding or
	// subtracting bits, which were lost due to rounding, to the latest layer.
	if (TotalBitrateBps > Sum)
	{
		Bitrates.Last() += TotalBitrateBps - Sum;
	}
	else if (TotalBitrateBps < Sum)
	{
		Bitrates.Last() -= Sum - TotalBitrateBps;
	}

	return Bitrates;
}

// Returns the minimum bitrate needed for `NumActiveLayers` Spatial Layers to
// become Active using the configuration specified by `Config`.
uint32 FindLayerTogglingThreshold(const FVideoEncoderConfig &Config, size_t FirstActiveLayer, size_t NumActiveLayers)
{
	if (NumActiveLayers == 1)
	{
		return Config.SpatialLayers[0].MinBitrate * 1000;
	}

	uint32 LowerBound = 0;
	uint32 UpperBound = 0;
	if (NumActiveLayers > 1)
	{
		for (size_t i = 0; i < NumActiveLayers - 1; ++i)
		{
			LowerBound += Config.SpatialLayers[FirstActiveLayer + i].MinBitrate * 1000;
			UpperBound += Config.SpatialLayers[FirstActiveLayer + i].MaxBitrate * 1000;
		}
	}
	UpperBound += Config.SpatialLayers[FirstActiveLayer + NumActiveLayers - 1].MinBitrate * 1000;

	// Do a binary search until upper and lower bound is the highest bitrate for
	// `NumActiveLayers` - 1 Layers and lowest bitrate for `NumActiveLayers`
	// Layers respectively.
	while (UpperBound - LowerBound > 1.f)
	{
		uint32 TryRate = (LowerBound + UpperBound) / 2;
		if (AdjustAndVerify(Config, FirstActiveLayer, SplitBitrate(NumActiveLayers, TryRate, SpatialLayeringRateScalingFactor)).Num() == NumActiveLayers)
		{
			UpperBound = TryRate;
		}
		else
		{
			LowerBound = TryRate;
		}
	}
	return UpperBound;
}

FVideoBitrateAllocatorSVC::FNumLayers FVideoBitrateAllocatorSVC::GetNumLayers(const FVideoEncoderConfig &Config)
{
	FNumLayers Layers;
	if (Config.ScalabilityMode != EScalabilityMode::None)
	{
		if (TUniquePtr<FScalableVideoController> Structure = CreateScalabilityStructure(Config.ScalabilityMode))
		{
			FScalableVideoController::FStreamLayersConfig StreamConfig = Structure->StreamConfig();
			Layers.Spatial = StreamConfig.NumSpatialLayers;
			Layers.Temporal = StreamConfig.NumTemporalLayers;
			return Layers;
		}
	}

	if (Config.Codec == EVideoCodec::VP9)
	{
		Layers.Spatial = Config.NumberOfSpatialLayers;
		Layers.Temporal = Config.NumberOfTemporalLayers;
		return Layers;
	}

	Layers.Spatial = 1;
	Layers.Temporal = 1;
	return Layers;
}

FVideoBitrateAllocatorSVC::FVideoBitrateAllocatorSVC(const FVideoEncoderConfig &Config)
	: Config(Config), NumLayers(GetNumLayers(Config)), CumulativeLayerStartBitrates(GetLayerStartBitrates(Config)), LastActiveLayerCount(0)
{
	check(NumLayers.Spatial > 0);
	check(NumLayers.Spatial <= 5);
	check(NumLayers.Temporal > 0);
	check(NumLayers.Temporal <= 3);
	for (size_t LayerIdx = 0; LayerIdx < NumLayers.Spatial; ++LayerIdx)
	{
		// Verify min <= target <= max.
		if (Config.SpatialLayers[LayerIdx].bActive)
		{
			check(Config.SpatialLayers[LayerIdx].MaxBitrate > 0);
			check(Config.SpatialLayers[LayerIdx].MaxBitrate >= Config.SpatialLayers[LayerIdx].MinBitrate);
			check(Config.SpatialLayers[LayerIdx].TargetBitrate >= Config.SpatialLayers[LayerIdx].MinBitrate);
			check(Config.SpatialLayers[LayerIdx].MaxBitrate >= Config.SpatialLayers[LayerIdx].TargetBitrate);
		}
	}
}

FVideoBitrateAllocation FVideoBitrateAllocatorSVC::Allocate(FVideoBitrateAllocationParameters Parameters)
{
	uint32 TotalBitrateBps = Parameters.TotalBitrateBps;
	if (Config.MaxBitrate != 0)
	{
		TotalBitrateBps = FMath::Min<uint32>(TotalBitrateBps, Config.MaxBitrate * 1000);
	}

	if (Config.SpatialLayers[0].TargetBitrate == 0)
	{
		// Delegate rate distribution to encoder wrapper if bitrate thresholds
		// are not set.
		FVideoBitrateAllocation BitrateAllocation;
		BitrateAllocation.SetBitrate(0, 0, TotalBitrateBps);
		return BitrateAllocation;
	}

	const FActiveSpatialLayers ActiveLayers = GetActiveSpatialLayers(Config, NumLayers.Spatial);
	size_t NumSpatialLayers = ActiveLayers.Num;

	NumSpatialLayers = FindNumEnabledLayers(Parameters.TotalBitrateBps);
	LastActiveLayerCount = NumSpatialLayers;

	FVideoBitrateAllocation Allocation = GetAllocationNormalVideo(TotalBitrateBps, ActiveLayers.First, NumSpatialLayers);
	Allocation.SetBwLimited(NumSpatialLayers < ActiveLayers.Num);
	return Allocation;
}

FVideoBitrateAllocation FVideoBitrateAllocatorSVC::GetAllocationNormalVideo(uint32 TotalBitrateBps, size_t FirstActiveLayer, size_t NumSpatialLayers) const
{
	TArray<uint32> SpatialLayerRates;
	if (NumSpatialLayers == 0)
	{
		// Not enough rate for even the base layer. Force Allocation at the total
		// bitrate anyway.
		NumSpatialLayers = 1;
		SpatialLayerRates.Add(TotalBitrateBps);
	}
	else
	{
		SpatialLayerRates = AdjustAndVerify(Config, FirstActiveLayer, SplitBitrate(NumSpatialLayers, TotalBitrateBps, SpatialLayeringRateScalingFactor));
		check(SpatialLayerRates.Num() == NumSpatialLayers);
	}

	FVideoBitrateAllocation BitrateAllocation;

	for (size_t sl_idx = 0; sl_idx < NumSpatialLayers; ++sl_idx)
	{
		TArray<uint32> TemporalLayerRates = SplitBitrate(NumLayers.Temporal, SpatialLayerRates[sl_idx], TemporalLayeringRateScalingFactor);

		// Distribute rate across Temporal Layers. Allocate more bits to lower
		// Layers since they are used for prediction of higher Layers and their
		// references are far apart.
		if (NumLayers.Temporal == 1)
		{
			BitrateAllocation.SetBitrate(sl_idx + FirstActiveLayer, 0, TemporalLayerRates[0]);
		}
		else if (NumLayers.Temporal == 2)
		{
			BitrateAllocation.SetBitrate(sl_idx + FirstActiveLayer, 0, TemporalLayerRates[1]);
			BitrateAllocation.SetBitrate(sl_idx + FirstActiveLayer, 1, TemporalLayerRates[0]);
		}
		else
		{
			check(NumLayers.Temporal == 3);
			// In case of three Temporal Layers the high layer has two frames and the
			// middle layer has one frame within GOP (in between two consecutive low
			// layer frames). Thus high layer requires more bits (comparing pure
			// bitrate of layer, excluding bitrate of base Layers) to keep quality on
			// par with lower Layers.
			BitrateAllocation.SetBitrate(sl_idx + FirstActiveLayer, 0, TemporalLayerRates[2]);
			BitrateAllocation.SetBitrate(sl_idx + FirstActiveLayer, 1, TemporalLayerRates[0]);
			BitrateAllocation.SetBitrate(sl_idx + FirstActiveLayer, 2, TemporalLayerRates[1]);
		}
	}

	return BitrateAllocation;
}

size_t FVideoBitrateAllocatorSVC::FindNumEnabledLayers(uint32 TargetRate) const
{
	if (CumulativeLayerStartBitrates.IsEmpty())
	{
		return 0;
	}

	size_t NumEnabledLayers = 0;
	for (uint32 StartRate : CumulativeLayerStartBitrates)
	{
		// First layer is always enabled.
		if (NumEnabledLayers == 0 || StartRate <= TargetRate)
		{
			++NumEnabledLayers;
		}
		else
		{
			break;
		}
	}

	return NumEnabledLayers;
}

uint32 FVideoBitrateAllocatorSVC::GetMaxBitrate(const FVideoEncoderConfig &Config)
{
	const FNumLayers NumLayers = GetNumLayers(Config);
	const FActiveSpatialLayers ActiveLayers = GetActiveSpatialLayers(Config, NumLayers.Spatial);

	uint32 MaxBitrate = 0.f;
	for (size_t sl_idx = 0; sl_idx < ActiveLayers.Num; ++sl_idx)
	{
		MaxBitrate += Config.SpatialLayers[ActiveLayers.First + sl_idx].MaxBitrate * 1000;
	}

	if (Config.MaxBitrate != 0)
	{
		MaxBitrate = FMath::Min<uint32>(MaxBitrate, Config.MaxBitrate * 1000);
	}

	return MaxBitrate;
}

uint32 FVideoBitrateAllocatorSVC::GetPaddingBitrate(const FVideoEncoderConfig &Config)
{
	auto StartBitrate = GetLayerStartBitrates(Config);
	if (StartBitrate.IsEmpty())
	{
		return 0; // All Layers are deactivated.
	}

	return StartBitrate.Last();
}

TArray<uint32> FVideoBitrateAllocatorSVC::GetLayerStartBitrates(const FVideoEncoderConfig &Config)
{
	TArray<uint32> StartBitrates;
	const FNumLayers NumLayers = GetNumLayers(Config);
	const FActiveSpatialLayers ActiveLayers = GetActiveSpatialLayers(Config, NumLayers.Spatial);
	uint32 LastRate = 0.f;
	for (size_t i = 1; i <= ActiveLayers.Num; ++i)
	{
		uint32 LayerTogglingRate = FindLayerTogglingThreshold(Config, ActiveLayers.First, i);
		StartBitrates.Add(LayerTogglingRate);
		check(LastRate <= LayerTogglingRate);
		LastRate = LayerTogglingRate;
	}
	return StartBitrates;
}