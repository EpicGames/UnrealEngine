// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/VideoBitrateAllocation.h"

FVideoBitrateAllocation::FVideoBitrateAllocation()
	: SumBps(0)
	, bIsBwLimited(false)
{
}

bool FVideoBitrateAllocation::HasBitrate(uint64 SpatialIndex, uint64 TemporalIndex) const
{
	return Bitrates[SpatialIndex][TemporalIndex].IsSet();
}

uint32 FVideoBitrateAllocation::GetBitrate(uint64 SpatialIndex, uint64 TemporalIndex) const
{
	return Bitrates[SpatialIndex][TemporalIndex].Get(0);
}

bool FVideoBitrateAllocation::IsSpatialLayerUsed(uint64 SpatialIndex) const
{
	for (size_t i = 0; i < 4; ++i)
	{
		if (Bitrates[SpatialIndex][i].IsSet())
		{
			return true;
		}
	}
	return false;
}

uint32 FVideoBitrateAllocation::GetSpatialLayerSumBitrate(uint64 SpatialIndex) const
{
	return GetTemporalLayerSumBitrate(SpatialIndex, 3);
}

uint32 FVideoBitrateAllocation::GetTemporalLayerSumBitrate(uint64 SpatialIndex, uint64 TemporalIndex) const
{
	uint32 TemporalSum = 0;
	for (size_t i = 0; i <= TemporalIndex; ++i)
	{
		TemporalSum += Bitrates[SpatialIndex][i].Get(0);
	}
	return TemporalSum;
}

TArray<uint32> FVideoBitrateAllocation::GetTemporalLayerAllocation(uint64 SpatialIndex) const
{
	TArray<uint32> TemporalRates;

	// Find the highest temporal layer with a defined bitrate in order to
	// determine the size of the temporal layer allocation.
	for (size_t i = 4; i > 0; --i)
	{
		if (Bitrates[SpatialIndex][i - 1].IsSet())
		{
			TemporalRates.SetNum(i);
			break;
		}
	}

	for (size_t i = 0; i < TemporalRates.Num(); ++i)
	{
		TemporalRates[i] = Bitrates[SpatialIndex][i].Get(0);
	}

	return TemporalRates;
}

uint32 FVideoBitrateAllocation::GetSumBps() const
{
	return SumBps;
}

bool FVideoBitrateAllocation::IsBwLimited() const
{
	return bIsBwLimited;
}

bool FVideoBitrateAllocation::SetBitrate(size_t SpatialIndex, size_t TemporalIndex, uint32 BitrateBps)
{
	int64			   NewBitrateSumBps = SumBps;
	TOptional<uint32>& LayerBitrate = Bitrates[SpatialIndex][TemporalIndex];
	if (LayerBitrate)
	{
		NewBitrateSumBps -= *LayerBitrate;
	}
	NewBitrateSumBps += BitrateBps;
	if (NewBitrateSumBps > MaxBitrateBps)
	{
		return false;
	}

	LayerBitrate = BitrateBps;
	SumBps = static_cast<uint32>(NewBitrateSumBps);
	return true;
}

void FVideoBitrateAllocation::SetBwLimited(bool Limited)
{
	bIsBwLimited = Limited;
}

TArray<TOptional<FVideoBitrateAllocation>> FVideoBitrateAllocation::GetSimulcastAllocations() const
{
	TArray<TOptional<FVideoBitrateAllocation>> SimulcastBitrates;
	for (size_t si = 0; si < 5; ++si)
	{
		TOptional<FVideoBitrateAllocation> LayerBitrate;
		if (IsSpatialLayerUsed(si))
		{
			LayerBitrate = FVideoBitrateAllocation();
			for (int tl = 0; tl < 4; ++tl)
			{
				if (HasBitrate(si, tl))
				{
					LayerBitrate->SetBitrate(0, tl, GetBitrate(si, tl));
				}
			}
		}
		SimulcastBitrates.Add(LayerBitrate);
	}
	return SimulcastBitrates;
}