// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/VideoBitrateAllocator.h"

FVideoBitrateAllocation FVideoBitrateAllocator::GetAllocation(uint32 TotalBitrateBps, FFrameRate Framerate)
{
	return Allocate({ TotalBitrateBps, TotalBitrateBps, Framerate });
}

FVideoBitrateAllocation FVideoBitrateAllocator::Allocate(FVideoBitrateAllocationParameters Parameters)
{
	return GetAllocation(Parameters.TotalBitrateBps, Parameters.Framerate);
}