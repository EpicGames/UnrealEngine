// Copyright Epic Games, Inc. All Rights Reserved.

#if AVCODECS_USE_D3D

#include "Video/Resources/D3D/VideoResourceD3D.h"

#include "AVResult.h"

REGISTER_TYPEID(FVideoContextD3D11);
REGISTER_TYPEID(FVideoContextD3D12);
REGISTER_TYPEID(FVideoResourceD3D11);
REGISTER_TYPEID(FVideoResourceD3D12);

static TAVResult<EVideoFormat> ConvertFormat(DXGI_FORMAT Format)
{
	switch (Format)
	{
	case DXGI_FORMAT::DXGI_FORMAT_B8G8R8A8_UNORM:
		return EVideoFormat::BGRA;
	case DXGI_FORMAT::DXGI_FORMAT_R10G10B10A2_UNORM:
		return EVideoFormat::ABGR10;
	case DXGI_FORMAT::DXGI_FORMAT_NV12:
		return EVideoFormat::NV12;
	case DXGI_FORMAT::DXGI_FORMAT_R8_UNORM:
	case DXGI_FORMAT::DXGI_FORMAT_R8_UINT:
		return EVideoFormat::R8;
	default:
		return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("D3D format %d is not supported"), Format), TEXT("D3D"));
	}
}

FVideoContextD3D11::FVideoContextD3D11(TRefCountPtr<ID3D11Device> const& Device)
	: Device(Device)
{
}

FVideoContextD3D12::FVideoContextD3D12(TRefCountPtr<ID3D12Device> const& Device)
	: Device(Device)
{
}

FVideoDescriptor FVideoResourceD3D11::GetDescriptorFrom(TSharedRef<FAVDevice> const& Device, TRefCountPtr<ID3D11Texture2D> const& Raw)
{
	D3D11_TEXTURE2D_DESC RawDesc;
	Raw->GetDesc(&RawDesc);

	return FVideoDescriptor(ConvertFormat(RawDesc.Format), RawDesc.Width, RawDesc.Height);
}

FVideoResourceD3D11::FVideoResourceD3D11(TSharedRef<FAVDevice> const& Device, TRefCountPtr<ID3D11Texture2D> const& Raw, FAVLayout const& Layout)
	: TVideoResource(Device, Layout, GetDescriptorFrom(Device, Raw))
	, Raw(Raw)
{
	TRefCountPtr<IDXGIResource> DXGI = nullptr;

	HRESULT Result = Raw->QueryInterface(DXGI.GetInitReference());
	if (FAILED(Result))
	{
		FAVResult::Log(EAVResult::ErrorMapping, TEXT("D3D11 resource is not a DXGI resource"), TEXT("D3D11"), Result);

		return;
	}
	
	Result = DXGI->GetSharedHandle(&SharedHandle);
	if (FAILED(Result) || SharedHandle == nullptr)
	{
		FAVResult::Log(EAVResult::ErrorMapping, TEXT("Failed to share D3D11 resource (ensure it was created with the Shared flag)"), TEXT("D3D11"), Result);

		return;
	}
}

FAVResult FVideoResourceD3D11::Validate() const
{
	if (!Raw.IsValid())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Raw resource is invalid"), TEXT("D3D11"));
	}

	return EAVResult::Success;
}

FVideoDescriptor FVideoResourceD3D12::GetDescriptorFrom(TSharedRef<FAVDevice> const& Device, FRawD3D12 const& Raw)
{
	D3D12_RESOURCE_DESC const& RawDesc = Raw.D3DResource->GetDesc();

	return FVideoDescriptor(ConvertFormat(RawDesc.Format), RawDesc.Width, RawDesc.Height);
}

FVideoResourceD3D12::FVideoResourceD3D12(TSharedRef<FAVDevice> const& Device, FRawD3D12 const& Raw, FAVLayout const& Layout)
	: FVideoResourceD3D12(Device, Raw, Layout, GetDescriptorFrom(Device, Raw))
{
}

FVideoResourceD3D12::FVideoResourceD3D12(TSharedRef<FAVDevice> const& Device, FRawD3D12 const& Raw, FAVLayout const& Layout, FVideoDescriptor const& Descriptor)
	: TVideoResource(Device, Layout, Descriptor)
	, Raw(Raw)
{
	// Get the Heap Shared Handle otherwise get the Resource Shared Handle
	if (Raw.D3DHeap.IsValid() && !this->Raw.D3DHeapShared)
	{
		HRESULT const Result = this->GetContext()->Device->CreateSharedHandle(Raw.D3DHeap, nullptr, GENERIC_ALL, nullptr, &this->Raw.D3DHeapShared);
		if (FAILED(Result) || this->Raw.D3DHeapShared == nullptr)
		{
			FAVResult::Log(EAVResult::ErrorCreating, TEXT("Failed to share D3D12 Heap (ensure it was created with the Shared flag)"), TEXT("D3D12"), Result);
		}
	}
	else if (Raw.D3DResource.IsValid() && !this->Raw.D3DResourceShared)
	{
		HRESULT const Result = this->GetContext()->Device->CreateSharedHandle(Raw.D3DResource, nullptr, GENERIC_ALL, nullptr, &this->Raw.D3DResourceShared);
		if (FAILED(Result) || this->Raw.D3DResourceShared == nullptr)
		{
			FAVResult::Log(EAVResult::ErrorCreating, TEXT("Failed to share D3D12 Resource ensure that it is a commited Resource"), TEXT("D3D12"), Result);
		}
	}
	else
	{
		FAVResult::Log(EAVResult::ErrorCreating, TEXT("No shareable resource was passed into FVideoResourceD3D12"), TEXT("D3D12"));
	}

	// Get the Fences Shared Handle
	if (Raw.D3DFence.IsValid() && !this->Raw.D3DFenceShared)
	{
		HRESULT const Result = this->GetContext()->Device->CreateSharedHandle(Raw.D3DFence, nullptr, GENERIC_ALL, nullptr, &this->Raw.D3DFenceShared);
		if (FAILED(Result) || this->Raw.D3DFenceShared == nullptr)
		{
			FAVResult::Log(EAVResult::ErrorCreating, TEXT("Failed to share D3D12 Fence (ensure it was created with the Shared flag)"), TEXT("D3D12"), Result);
		}
	}
}

FVideoResourceD3D12::~FVideoResourceD3D12()
{
	if (Raw.D3DResource.IsValid() && Raw.D3DResourceShared)
	{
		BOOL const Result = CloseHandle(Raw.D3DResourceShared);
		if (!Result)
		{
			FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to close shared D3D12 resource handle"), TEXT("D3D12"), Result);
		}
	}

	if (Raw.D3DHeap.IsValid() && Raw.D3DHeapShared)
	{
		BOOL const Result = CloseHandle(Raw.D3DHeapShared);
		if (!Result)
		{
			FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to close shared D3D12 heap handle"), TEXT("D3D12"), Result);
		}
	}

	if (Raw.D3DFence.IsValid() && Raw.D3DFenceShared)
	{
		BOOL const Result = CloseHandle(Raw.D3DFenceShared);
		if (!Result)
		{
			FAVResult::Log(EAVResult::ErrorDestroying, TEXT("Failed to close shared D3D12 fence handle"), TEXT("D3D12"), Result);
		}
	}
}

uint64 FVideoResourceD3D12::GetSizeInBytes()
{
	const D3D12_RESOURCE_DESC Desc = Raw.D3DResource->GetDesc();
	const D3D12_RESOURCE_ALLOCATION_INFO AllocationInfo = this->GetContext()->Device->GetResourceAllocationInfo(0, 1, &Desc);
	if (AllocationInfo.SizeInBytes == UINT64_MAX)
	{
		FAVResult::Log(EAVResult::ErrorInvalidState, TEXT("D3D12 GetResourceAllocationInfo failed - likely a resource was requested that has invalid allocation info (e.g. is an invalid texture size)"), TEXT("D3D12"));
	}

	return static_cast<uint64>(AllocationInfo.SizeInBytes);
}

FAVResult FVideoResourceD3D12::Validate() const
{
	if (!Raw.D3DResource.IsValid())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Raw resource is invalid"), TEXT("D3D12"));
	}

	return EAVResult::Success;
}

#endif
