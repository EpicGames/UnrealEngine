// Copyright Epic Games, Inc. All Rights Reserved.

#if AVCODECS_USE_METAL

#include "Video/Resources/Metal/VideoResourceMetal.h"

THIRD_PARTY_INCLUDES_START
#include <VideoToolbox/VideoToolbox.h>
#include <CoreMedia/CMSync.h>
THIRD_PARTY_INCLUDES_END

REGISTER_TYPEID(FVideoContextMetal);
REGISTER_TYPEID(FVideoResourceMetal);

static TAVResult<EVideoFormat> ConvertFormat(MTL::PixelFormat Format)
{
	switch (Format)
	{
		case MTL::PixelFormatBGRA8Unorm:
		case MTL::PixelFormatBGRA8Unorm_sRGB:
			return EVideoFormat::BGRA;
		case MTL::PixelFormatBGR10A2Unorm:
			return EVideoFormat::ABGR10;
		default:
			return FAVResult(EAVResult::ErrorUnsupported, FString::Printf(TEXT("MTL::PixelFormat format %lld is not supported"), Format), TEXT("Metal"));
	}
}

static TAVResult<uint32> GetBytesPerPixelFromOSType(OSType Format)
{
	switch (Format)
	{
		case kCVPixelFormatType_OneComponent8:
		case kCVPixelFormatType_420YpCbCr8BiPlanarVideoRange:
		case kCVPixelFormatType_420YpCbCr8BiPlanarFullRange:
			return 1; 	
		case kCVPixelFormatType_OneComponent16Half:
		case kCVPixelFormatType_OneComponent16:
		case kCVPixelFormatType_422YpCbCr8:         
		case kCVPixelFormatType_422YpCbCr8_yuvs:    
		case kCVPixelFormatType_422YpCbCr10:        
		case kCVPixelFormatType_420YpCbCr10BiPlanarVideoRange:
		case kCVPixelFormatType_420YpCbCr10BiPlanarFullRange:
		case kCVPixelFormatType_DepthFloat16:
		case kCVPixelFormatType_14Bayer_GRBG:
		case kCVPixelFormatType_14Bayer_RGGB:
		case kCVPixelFormatType_14Bayer_BGGR:
		case kCVPixelFormatType_14Bayer_GBRG:
		case kCVPixelFormatType_DisparityFloat16:
			return 2;
		case kCVPixelFormatType_24RGB:
			return 3;
		case kCVPixelFormatType_OneComponent32Float:
		case kCVPixelFormatType_TwoComponent16Half:
		case kCVPixelFormatType_TwoComponent16:
		case kCVPixelFormatType_30RGB:
		case kCVPixelFormatType_32RGBA:
		case kCVPixelFormatType_32BGRA:
		case kCVPixelFormatType_32ARGB:
		case kCVPixelFormatType_32ABGR:
		case kCVPixelFormatType_DepthFloat32:
		case kCVPixelFormatType_DisparityFloat32:
			return 4;
		case kCVPixelFormatType_48RGB: 
			return 6;
		case kCVPixelFormatType_TwoComponent32Float:
			return 8;
		default:
			return FAVResult(EAVResult::ErrorUnsupported, TEXT("PixelFormat format is not supported"), TEXT("Metal"));

	}
}

FVideoContextMetal::FVideoContextMetal(MTL::Device* Device)
	: Device(Device)
{
}

FVideoDescriptor FVideoResourceMetal::GetDescriptorFrom(TSharedRef<FAVDevice> const& Device, MTL::Texture* Raw)
{
	uint32_t Width = Raw->width();
	uint32_t Height = Raw->height();
	TAVResult<EVideoFormat> ConvertedFormat = ConvertFormat(Raw->pixelFormat());
	
	return FVideoDescriptor(ConvertedFormat, Width, Height);
}

FVideoResourceMetal::FVideoResourceMetal(TSharedRef<FAVDevice> const& Device, MTL::Texture* Raw, FAVLayout const& Layout)
	: TVideoResource(Device, Layout, GetDescriptorFrom(Device, Raw))
	, Raw(Raw)
	, StagingBuffer(nullptr)
{
}

FVideoResourceMetal::~FVideoResourceMetal()
{
	if (StagingBuffer != nullptr)
	{
		StagingBuffer->release();
		StagingBuffer = nullptr;
	}
}

FAVResult FVideoResourceMetal::CopyFrom(CVPixelBufferRef Other)
{	
	// TODO (william.belcher): Support planar pixel buffers
	if (CVPixelBufferIsPlanar(Other))
	{
		return FAVResult(EAVResult::ErrorUnsupported, TEXT("Planar pixel buffers are not supported for copying to Metal resources"), TEXT("Metal"));
	}

	const size_t SrcHeight = CVPixelBufferGetHeight(Other);
	const size_t SrcWidth = CVPixelBufferGetWidth(Other);
	
	if (SrcHeight != GetDescriptor().Height || SrcWidth != GetDescriptor().Width)
	{
		return FAVResult(EAVResult::Error, TEXT("PixelBuffer dimensions must match Metal resource dimensions!"), TEXT("Metal"));
	}
	
	TAVResult<uint32> BytesPerPixel = GetBytesPerPixelFromOSType(CVPixelBufferGetPixelFormatType(Other));
	if (BytesPerPixel.IsNotSuccess())
	{
		return BytesPerPixel;
	}
		
	const size_t TightRowSize = static_cast<size_t>(BytesPerPixel) * SrcWidth;
	const size_t BufferSize = TightRowSize * SrcHeight;

	// Create staging buffer for the copy
	if (StagingBuffer != nullptr && StagingBuffer->length() != BufferSize)
	{
		// We expect the FVideoResourceMetal to be recreated if the size ever changes, but we check here just to make sure
		StagingBuffer->release();
		StagingBuffer = nullptr;
	}

	if (StagingBuffer == nullptr)
	{
		StagingBuffer = GetContext()->Device->newBuffer(BufferSize, MTL::ResourceStorageModeShared);
		if (StagingBuffer == nullptr)
		{
			return FAVResult(EAVResult::Error, TEXT("Failed to create staging buffer"), TEXT("Metal"));
		}
	}

	if (const CVReturn Result = CVPixelBufferLockBaseAddress(Other, 0); Result != kCVReturnSuccess)
	{
		return FAVResult(EAVResult::Error, TEXT("Failed to lock input pixel buffer!"), TEXT("Metal"));
	}

	const size_t SrcStride = CVPixelBufferGetBytesPerRow(Other);
	const uint8* SrcPtr = static_cast<const uint8*>(CVPixelBufferGetBaseAddress(Other));

	uint8* DstPtr = static_cast<uint8*>(StagingBuffer->contents());
	for (size_t Row = 0; Row < SrcHeight; Row++)
	{
		const uint8* SrcRow = SrcPtr + Row * SrcStride;
		uint8* DstRow = DstPtr + Row * TightRowSize;

		FMemory::Memcpy(DstRow, SrcRow, TightRowSize);
	}

	Raw->replaceRegion(MTL::Region(0, 0, SrcWidth, SrcHeight), 0, 0, StagingBuffer->contents(), TightRowSize, 0);
	
	CVPixelBufferUnlockBaseAddress(Other, 0);
	
	return EAVResult::Success;
}

FAVResult FVideoResourceMetal::Validate() const
{
	if (!Raw)
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Raw resource is invalid"), TEXT("Metal"));
	}

	return EAVResult::Success;
}

#endif
