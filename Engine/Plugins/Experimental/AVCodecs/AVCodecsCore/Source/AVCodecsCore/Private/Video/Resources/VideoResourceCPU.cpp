// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Resources/VideoResourceCPU.h"

REGISTER_TYPEID(FVideoContextCPU);
REGISTER_TYPEID(FVideoResourceCPU);
REGISTER_TYPEID(FResolvableVideoResourceCPU);

FVideoContextCPU::FVideoContextCPU()
{
}

FVideoResourceCPU::FVideoResourceCPU(TSharedRef<FAVDevice> const& Device, TSharedPtr<uint8> const& Raw, FAVLayout const& Layout, FVideoDescriptor const& Descriptor)
	: TVideoResource(Device, Layout, Descriptor)
	, Raw(Raw)
{
}

FAVResult FVideoResourceCPU::Validate() const
{
	if (!Raw.IsValid())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Raw resource is invalid"), TEXT("CPU"));
	}

	return EAVResult::Success;
}

TSharedPtr<FVideoResourceCPU> FResolvableVideoResourceCPU::TryResolve(TSharedPtr<FAVDevice> const& Device, FVideoDescriptor const& Descriptor)
{
	return MakeShared<FVideoResourceCPU>(Device.ToSharedRef(), nullptr, FAVLayout(GetStrideInterleavedOrLuma(Descriptor), 0, GetSize(Descriptor)), Descriptor);
}

uint32 FResolvableVideoResourceCPU::GetStrideInterleavedOrLuma(FVideoDescriptor const& Descriptor)
{
	switch (Descriptor.Format)
	{
		case EVideoFormat::BGRA:
		case EVideoFormat::ABGR10:
		case EVideoFormat::YUV420:
		case EVideoFormat::YUV444:
		case EVideoFormat::NV12:
			return Descriptor.Width;
		case EVideoFormat::YUV444_16:
		case EVideoFormat::P010:
			return Descriptor.Width * 2;
		case EVideoFormat::R8:
			// TODO (william.belcher)
			return 0;
		case EVideoFormat::G16:
			// TODO (william.belcher)
			return 0;
		default:
			return 0;
	}
}

uint32 FResolvableVideoResourceCPU::GetSize(FVideoDescriptor const& Descriptor)
{
	switch (Descriptor.Format)
	{
		case EVideoFormat::BGRA:
		case EVideoFormat::ABGR10:
			return Descriptor.Width * Descriptor.Height * 4;
		case EVideoFormat::YUV420:
			return Descriptor.Width * Descriptor.Height + (2 * ((Descriptor.Width + 1) / 2) * ((Descriptor.Height + 1) / 2));
		case EVideoFormat::YUV444:
			return Descriptor.Width * Descriptor.Height * 3;
		case EVideoFormat::YUV444_16:
			return Descriptor.Width * Descriptor.Height * 6;
		case EVideoFormat::NV12:
			return Descriptor.Width * Descriptor.Height + (Descriptor.Width * ((Descriptor.Height + 1) / 2));
		case EVideoFormat::P010:
			return Descriptor.Width * Descriptor.Height + (2 * ((Descriptor.Width + 1) / 2) * ((Descriptor.Height + 1) / 2)) * 2;
		case EVideoFormat::R8:
			// TODO (william.belcher)
			return 0;
		case EVideoFormat::G16:
			// TODO (william.belcher)
			return 0;
		default:
			return 0;
	}
}
