// Copyright Epic Games, Inc. All Rights Reserved.

#if AVCODECS_USE_VULKAN

#include "Video/Resources/Vulkan/VideoResourceVulkan.h"
#include "HAL/Platform.h"

#include "vulkan/vulkan_core.h"

#if PLATFORM_WINDOWS
#include "vulkan/vulkan_win32.h"
#endif

REGISTER_TYPEID(FVideoContextVulkan);
REGISTER_TYPEID(FVideoResourceVulkan);

FVideoContextVulkan::FVideoContextVulkan(VkInstance const& Instance, VkDevice const& Device, VkPhysicalDevice const& PhysicalDevice, TFunction<PFN_vkVoidFunction(const char*)> const& GetDeviceProcAddrFunc)
	: Instance(Instance)
	, Device(Device)
	, PhysicalDevice(PhysicalDevice)
	, vkGetDeviceProcAddr(GetDeviceProcAddrFunc)
{
}

FVideoResourceVulkan::FVideoResourceVulkan(TSharedRef<FAVDevice> const& Device, VkDeviceMemory Raw, FAVLayout const& Layout, FVideoDescriptor const& Descriptor)
	: TVideoResource(Device, Layout, Descriptor)
	, DeviceMemory(Raw)
{
#if PLATFORM_WINDOWS
	VkMemoryGetWin32HandleInfoKHR MemoryGetHandleInfoKHR = {};
	MemoryGetHandleInfoKHR.sType = VK_STRUCTURE_TYPE_MEMORY_GET_WIN32_HANDLE_INFO_KHR;
	MemoryGetHandleInfoKHR.pNext = nullptr;
	MemoryGetHandleInfoKHR.memory = Raw;
	MemoryGetHandleInfoKHR.handleType = VK_EXTERNAL_MEMORY_HANDLE_TYPE_OPAQUE_WIN32_BIT;

PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	PFN_vkGetMemoryWin32HandleKHR const GetMemoryWin32HandleKHR = (PFN_vkGetMemoryWin32HandleKHR)GetContext()->vkGetDeviceProcAddr("vkGetMemoryWin32HandleKHR");
	//TODO (aidan) we need a verify vulkan result equivalent to VERIFYVULKANRESULT_EXTERNAL (used in Unreal) to make this call safer
	VkResult const Result = GetMemoryWin32HandleKHR(GetContext()->Device, &MemoryGetHandleInfoKHR, &DeviceMemorySharedHandle);
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

#else
	VkMemoryGetFdInfoKHR MemoryGetFdInfoKHR = {};	
	MemoryGetFdInfoKHR.sType = VK_STRUCTURE_TYPE_MEMORY_GET_FD_INFO_KHR;
	MemoryGetFdInfoKHR.pNext = nullptr;
	MemoryGetFdInfoKHR.memory = Raw;
	MemoryGetFdInfoKHR.handleType = VK_EXTERNAL_MEMORY_HANDLE_TYPE_OPAQUE_FD_BIT_KHR;

	PFN_vkGetMemoryFdKHR const GetMemoryFdKHR = (PFN_vkGetMemoryFdKHR)GetContext()->vkGetDeviceProcAddr( "vkGetMemoryFdKHR" );
	//TODO (aidan) we need a verify vulkan result equivalent to VERIFYVULKANRESULT_EXTERNAL (used in Unreal) to make this call safer
	VkResult const Result = GetMemoryFdKHR(GetContext()->Device, &MemoryGetFdInfoKHR, &DeviceMemorySharedHandle);
#endif

	if (Result != VK_SUCCESS || !DeviceMemorySharedHandle)
	{
		FAVResult::Log(EAVResult::ErrorCreating, TEXT("Failed to shared Vulkan resource (ensure it was created with the External flag)"), TEXT("Vulkan"), Result);

		return;
	}

	// TODO add PFN_vkGetFenceFdKHR calls for getting access to the resources fence
}

FAVResult FVideoResourceVulkan::Validate() const
{
	if (DeviceMemory == nullptr)
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Raw resource is invalid"), TEXT("Vulkan"));
	}

	return EAVResult::Success;
}
#endif
