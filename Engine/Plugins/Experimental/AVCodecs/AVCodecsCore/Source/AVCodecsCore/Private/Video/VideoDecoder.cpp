// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/VideoDecoder.h"

REGISTER_TYPEID(FVideoDecoder);
