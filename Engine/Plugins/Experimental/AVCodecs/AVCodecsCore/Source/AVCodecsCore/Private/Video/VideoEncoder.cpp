// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/VideoEncoder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VideoEncoder)

REGISTER_TYPEID(FVideoEncoder);

TMap<FString, EScalabilityMode> const ScalabilityModeMap = {
	{ "L1T1", EScalabilityMode::L1T1 },
	{ "L1T2", EScalabilityMode::L1T2 },
	{ "L1T3", EScalabilityMode::L1T3 },
	{ "L2T1", EScalabilityMode::L2T1 },
	{ "L2T1h", EScalabilityMode::L2T1h },
	{ "L2T1_KEY", EScalabilityMode::L2T1_KEY },
	{ "L2T2", EScalabilityMode::L2T2 },
	{ "L2T2h", EScalabilityMode::L2T2h },
	{ "L2T2_KEY", EScalabilityMode::L2T2_KEY },
	{ "L2T2_KEY_SHIFT", EScalabilityMode::L2T2_KEY_SHIFT },
	{ "L2T3", EScalabilityMode::L2T3 },
	{ "L2T3h", EScalabilityMode::L2T3h },
	{ "L2T3_KEY", EScalabilityMode::L2T3_KEY },
	{ "L3T1", EScalabilityMode::L3T1 },
	{ "L3T1h", EScalabilityMode::L3T1h },
	{ "L3T1_KEY", EScalabilityMode::L3T1_KEY },
	{ "L3T2", EScalabilityMode::L3T2 },
	{ "L3T2h", EScalabilityMode::L3T2h },
	{ "L3T2_KEY", EScalabilityMode::L3T2_KEY },
	{ "L3T3", EScalabilityMode::L3T3 },
	{ "L3T3h", EScalabilityMode::L3T3h },
	{ "L3T3_KEY", EScalabilityMode::L3T3_KEY },
	{ "S2T1", EScalabilityMode::S2T1 },
	{ "S2T1h", EScalabilityMode::S2T1h },
	{ "S2T2", EScalabilityMode::S2T2 },
	{ "S2T2h", EScalabilityMode::S2T2h },
	{ "S2T3", EScalabilityMode::S2T3 },
	{ "S2T3h", EScalabilityMode::S2T3h },
	{ "S3T1", EScalabilityMode::S3T1 },
	{ "S3T1h", EScalabilityMode::S3T1h },
	{ "S3T2", EScalabilityMode::S3T2 },
	{ "S3T2h", EScalabilityMode::S3T2h },
	{ "S3T3", EScalabilityMode::S3T3 },
	{ "S3T3h", EScalabilityMode::S3T3h },
	{ "None", EScalabilityMode::None }
};
