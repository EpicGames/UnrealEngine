// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/VideoResource.h"

FVideoResource::FVideoResource(TSharedRef<FAVDevice> const& Device, FAVLayout const& Layout, FVideoDescriptor const& Descriptor)
	: FAVResource(Device, Layout)
	, Descriptor(Descriptor)
{
}
