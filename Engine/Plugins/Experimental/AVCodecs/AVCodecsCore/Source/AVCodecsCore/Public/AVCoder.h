// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "AVConfig.h"
#include "AVExtension.h"
#include "AVResource.h"
#include "AVResult.h"

#define UE_API AVCODECSCORE_API

/**
 * HOW TO USE
 *
 * Terms:
 * - Domain: A specialized area of coding, such as 'video encoding', 'video decoding', 'audio decoding', 'subtitle encoding', etc.
 * - Configuration: An external collection of parameters that determine how the coder's underlying architecture is created and operates.
 * - Instance: The shared data cache of a coder. Contains all current state and configuration applied to a coder.
 * - Resource: An abstract wrapper to device memory used by a coder (can be on the CPU/GPU or an external device).
 * - Device: The physical hardware or software device that a coder operates on. Provides access to coder resources.
 * - Packet: A segment of a raw bitstream that contains any amount of frame data. Encoders create these and decoders consume them.
 *
 * TAVCoder uses an 'interleaved' inheritance model between itself and a 'domain', to support inheritance of select functionality (full, config-less, resource+config-less).
 *
 *	1)	TVideoDecoder<TResource, TConfig>							// Domain configuration (typed packets)
 *	2)		: TAVCoder<TVideoDecoder, TResource, TConfig>			// Common configuration (pending/apply/applied)
 *	3)			: TVideoDecoder<TResource>							// Domain resources (resolvable decode for decoders, encode/flush packets for encoders)
 *	4)				: TAVCoder<TVideoDecoder, TResource>			// Common resources (unused for now)
 *	5)					: TVideoDecoder<>							// Domain base (packets, flush packets for decoders)
 *	6)						: TAVCoder<TVideoDecoder>				// Common base (factory for domain)
 *	7)							: IAVCoder							// Interface base
 */

/**
 * Pre-declaration of template parameters to ensure void fallback in nearly all cases, which is then specialized for use with interleaving
 *
 * @tparam TDomain Type of the domain to inherit interleaved, expected to mirror this model, see above.
 * @tparam TResource Type of AVResource to operate on.
 * @tparam TConfig Type of AVConfig to operate by.
 */
template <template <typename TResource = void, typename TConfig = void> typename TDomain, typename TResource = void, typename TConfig = void>
class TAVCoder;

/**
 * Simple base coder interface
 */
class IAVCoder
{
protected:
	/**
	 * Type-erased factory list. First level is domain ids, second is resource ids, third is configuration ids.
	 * Stored void pointers point to a TArray of TAVCoder::TFactory's with types matching the domain, resource, and configuration ids.
	 * This is declared here to avoid exporting the template types (which can be real dodgy).
	 *
	 * @see FTypeID
	 * @see TAVCoder::GetFactories
	 */
	static UE_API TMap<FTypeID, TMap<FTypeID, TMap<FTypeID, TSharedPtr<void>>>> Factories;

public:
	virtual ~IAVCoder() = default;

	/**
	 * @return Get the physical device this coder runs on.
	 */
	virtual TSharedPtr<FAVDevice> const& GetDevice() const = 0;
	
	/**
	 * @return Get the shared data cache of this coder.
	 */
	virtual TSharedPtr<FAVInstance> const& GetInstance() const = 0;

	/**
	 * Force any pending configuration to apply.
	 * 
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult ApplyConfig() = 0;

	/**
	 * @return True if the underlying coder architecture is ready for use.
	 */
	virtual bool IsOpen() const = 0;

	/**
	 * Construct the underlying coder architecture on a physical device with a configuration.
	 * 
	 * @param NewDevice The physical device on which to construct this coder.
	 * @param NewInstance The cache with which to construct this coder.
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) = 0;

	/**
	 * Gracefully tear down the underlying coder architecture.
	 */
	virtual void Close() = 0;
};

/**
 * Common coder base with a factory, that supports typesafe resource handling and configuration.
 *
 * Usage: To be inherited by a domain implementation, see top of file.
 */
template <template <typename TResource = void, typename TConfig = void> typename TDomain, typename TResource, typename TConfig>
class TAVCoder : public TDomain<TResource>
{
public:
	using ConfigType = TConfig;
	
	/**
	 * Wrapper coder that transforms resource/config types for use with a differently typed child coder.
	 *
	 * @tparam TChildResource Type of child resource.
	 * @tparam TChildConfig Type of child config.
	 */
	template <typename TChildResource, typename TChildConfig>
	class TWrapper : public TDomain<TResource, TConfig>
	{
	public:
		/**
		 * The child coder to wrap.
		 */
		TSharedRef<TDomain<TChildResource, TChildConfig>> Child;

		TWrapper(TSharedRef<TDomain<TChildResource, TChildConfig>> const& Child)
			: Child(Child)
		{
			if (Child->IsOpen())
			{
				TDomain<TResource, TConfig>::Open(Child->GetDevice().ToSharedRef(), Child->GetInstance().ToSharedRef());

				FAVExtension::TransformConfig<TChildConfig, TConfig>(this->GetInstance()->template Edit<TChildConfig>(), this->GetPendingConfig());
			}
		}

		virtual bool IsOpen() const override
		{
			return Child->IsOpen();
		}

		virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override
		{
			if (this->IsOpen())
			{
				return FAVResult(EAVResult::ErrorInvalidState, TEXT("Coder already open"));
			}

			FAVResult Result = TDomain<TResource, TConfig>::Open(NewDevice, NewInstance);
			if (Result.IsNotSuccess())
			{
				return Result;
			}

			Result = FAVExtension::TransformConfig<TChildConfig, TConfig>(NewInstance->Edit<TChildConfig>(), this->GetPendingConfig());
			if (Result.IsNotSuccess())
			{
				return Result;
			}

			return Child->Open(NewDevice, NewInstance);
		}
		
		virtual void Close() override
		{
			if (this->IsOpen())
			{
				Child->Close();
			}
		}

		virtual FAVResult ApplyConfig() override
		{
			if (!this->IsOpen())
			{
				return FAVResult(EAVResult::ErrorInvalidState, TEXT("Coder not open"));
			}

			FAVResult const Result = TDomain<TResource, TConfig>::ApplyConfig();
			if (Result.IsNotSuccess())
			{
				return Result;
			}

			return FAVExtension::TransformConfig<TChildConfig, TConfig>(Child->EditPendingConfig(), this->GetAppliedConfig());
		}
	};

protected:
	/**
	 * The currently active configuration, for comparison with PendingConfig. Never modify this directly.
	 */
	TConfig AppliedConfig;

public:
	/**
	 * @return Get the currently active configuration, for comparison with PendingConfig.
	 */
	TConfig const& GetAppliedConfig() const
	{
		return AppliedConfig;
	}

	/**
	 * @return The pending configuration, to be applied on the next call to ApplyConfig.
	 */
	TConfig const& GetPendingConfig() const
	{
		return this->GetInstance()->template Get<TConfig>();
	}

	/**
	 * @return A mutable reference to the pending configuration, to be applied on the next call to ApplyConfig.
	 */
	TConfig& EditPendingConfig() const
	{
		return this->GetInstance()->template Edit<TConfig>();
	}

	/**
	 * Set the pending configuration, to be applied on the next call to ApplyConfig.
	 * 
	 * @param NewPendingConfig The pending configuration to be set.
	 */
	void SetPendingConfig(TConfig const& NewPendingConfig) const
	{
		this->EditPendingConfig() = NewPendingConfig;
	}

	/**
	 * Force any pending configuration to apply.
	 * 
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult ApplyConfig() override
	{
		AppliedConfig = GetPendingConfig();

		return EAVResult::Success;
	}
};

/**
 * Common coder base with a factory, that supports typesafe resource handling.
 *
 * Usage: To be inherited by a domain implementation, see top of file.
 */
template <template <typename TResource = void, typename TConfig = void> typename TDomain, typename TResource>
class TAVCoder<TDomain, TResource, void> : public TDomain<>
{
public:
	using ResourceType = TResource;
};

/**
 * Common coder base with a factory.
 *
 * Usage: To be inherited by a domain implementation, see top of file.
 */
template <template <typename TResource = void, typename TConfig = void> typename TDomain>
class TAVCoder<TDomain, void, void> : public IAVCoder
{
public:
	/**
	 * Delegate used to test factory compatibility.
	 */
	typedef TFunction<bool(TSharedRef<FAVDevice> const&, TSharedRef<FAVInstance> const&)> FIsCompatible;

	/**
	 * The individual factory type constructed by a call to Register.
	 */
	template <typename TResource, typename TConfig>
	struct TFactory final
	{
	public:
		typedef TFunction<TSharedPtr<TDomain<TResource, TConfig>>()> FCreate;

		FIsCompatible const IsCompatible;
		FCreate const Create;

		TFactory(FIsCompatible const& IsCompatible, FCreate const& Create)
			: IsCompatible(IsCompatible)
			, Create(Create)
		{
		}
	};

private:
	// Internal getter for type-specific coder factories.
	template <typename TResource, typename TConfig>
	static TArray<TFactory<TResource, TConfig>>& GetFactories()
	{
		// Same as void*, but memory safe because shared pointers store custom deleters (so void is still virtually destructible)
		// Worth the hack to reduce verbosity elsewhere, see below
		TSharedPtr<TArray<TFactory<TResource, TConfig>>>& Data = *(TSharedPtr<TArray<TFactory<TResource, TConfig>>>*)&Factories.FindOrAdd(FTypeID::Get<TDomain<>>()).FindOrAdd(FTypeID::Get<TResource>()).FindOrAdd(FTypeID::Get<TConfig>());
		if (!Data.IsValid())
		{
			Data = MakeShared<TArray<TFactory<TResource, TConfig>>>();
		}

		return *Data;

		/* ALTERNATE EXAMPLE: Can be handled type-safely by defining specialised GetFactories() in place of the current ENABLE_CODER_FACTORY macro, but much more verbose for maintainers.
		#define ENABLE_CODER_FACTORY(TDomain, TResource, TConfig) \
		template <> template <> \
		DLLEXPORT TArray<typename TAVCoder<TDomain>::TFactory<TResource, TConfig>>& TAVCoder<TDomain>::GetFactories() \
		{ \
			static TArray<typename TAVCoder<TDomain>::template TFactory<TResource, TConfig>> All; \
			return All; \
		}*/
	}

public:
	/**
	 * Wrap a fully typed child coder with a differently-typed outer encoder, to transform one resource/config type into another.
	 *
	 * @tparam TCoder Fully type-complete (ie. FVideoEncoderNVENC : TVideoEncoder<FVideoResourceCUDA, FVideoConfigNVENC>) coder.
	 * @tparam TResource Resource type of returned wrapper coder.
	 * @tparam TConfig Config type of returned wrapper coder.
	 * @return The new wrapper coder containing the child.
	 */
	template <typename TResource, typename TConfig, typename TCoder>
	static TSharedPtr<TDomain<TResource, TConfig>> Wrap(TSharedPtr<TCoder> const& Child)
	{
		if constexpr (std::is_same_v<TResource, typename TCoder::ResourceType> && std::is_same_v<TConfig, typename TCoder::ConfigType>)
		{
			return Child;
		}
		else
		{
			if (Child.IsValid())
			{
				typedef typename TDomain<TResource, TConfig>::template TWrapper<typename TCoder::ResourceType, typename TCoder::ConfigType> WrapperType;
				return MakeShared<WrapperType>(Child.ToSharedRef());
			}

			return nullptr;
		}
	}

	/**
	 * Register a fully type-complete coder with the domain factory.
	 *
	 * @tparam TCoder Fully type-complete (ie. FVideoEncoderNVENC : TVideoEncoder<FVideoResourceCUDA, FVideoConfigNVENC>) coder to be created by the factory. Must have a valid default constructor. Must be assignable to the base domain with the filter types below.
	 * @tparam TResource Type of AVResource to filter by when requested.
	 * @tparam TConfig Type of AVConfig to to filter by when requested.
	 * @param IsCompatible Optional delegate to further filter this coder.
	 * @return The constructed factory, to be further customised if desired.
	 */
	template <typename TCoder, typename TResource, typename TConfig>
	static TFactory<TResource, TConfig>& Register(FIsCompatible const& IsCompatible = nullptr)
	{
		return GetFactories<TResource, TConfig>().Emplace_GetRef(
			[IsCompatible](TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
			{
				return FAVExtension::IsCompatible<TCoder, TResource>(NewDevice) && FAVExtension::IsCompatible<TCoder, TConfig>(NewInstance) && (IsCompatible == nullptr || IsCompatible(NewDevice, NewInstance));
			},
			[]()
			{
				return Wrap<TResource, TConfig, TCoder>(MakeShared<TCoder>()).ToSharedRef();
			});
	}

	/**
	 * Register a partially type-complete coder with the domain factory, but with multiple filter configurations and permuted typing.
	 * Cannot provide additional custom filters, if this is necessary then you must directly call Register.
	 * 
	 * Usage: Highly recursive template permuter, see NVENCModule source for example of use.
	 */
	template <typename TCoder>
	struct RegisterPermutationsOf
	{
		template <typename TResource = void, typename... TResources>
		struct With
		{
			template <typename TConfig = void, typename... TConfigs>
			struct And
			{
				And(FIsCompatible const& IsCompatible = nullptr)
				{
					Register<TCoder, TResource, TConfig>(IsCompatible);

					typename With<TResources...>::template And<TConfig> const NextResource(IsCompatible);

					And<TConfigs...> const NextConfig(IsCompatible);
				}
			};

			With() = delete;

		private:
			// End case
			template <>
			struct And<void>
			{
				And(FIsCompatible const& IsCompatible = nullptr)
				{
				}
			};
		};

	private:
		// End case
		template <>
		struct With<void>
		{
			// End case
			template <typename...>
			struct And
			{
				And(FIsCompatible const& IsCompatible = nullptr)
				{
				}
			};
		};
	};

	/**
	 * Count the number of different coders that can be created to match this combination of resource and configuration.
	 *
	 * @tparam TResource Type of AVResource the coder will support.
	 * @tparam TConfig Type of AVConfig the coder will support.
	 * @param NewDevice Device to filter the coder against, and to open it with.
	 * @param NewInstance Instance to filter the coder against, and to open it with.
	 * @return The number of different coders that can be created.
	 */
	template <typename TResource, typename TConfig>
	static int32 CountSupported(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
	{
		int32 Result = 0;
		for (TFactory<TResource, TConfig> const& Factory : GetFactories<TResource, TConfig>())
		{
			if (Factory.IsCompatible(NewDevice, NewInstance))
			{
				++Result;
			}
		}

		return Result;
	}

	/**
	 * Checks if any coder can be created to match this combination of resource and configuration.
	 * This does NOT check the device or desired configuration, so all potential coders may still reject being created.
	 *
	 * @tparam TResource Type of AVResource the coder will support.
	 * @tparam TConfig Type of AVConfig the coder will support.
	 * @return True if a coder could be created.
	 */
	template <typename TResource, typename TConfig>
	static bool IsSupported()
	{
		return GetFactories<TResource, TConfig>().Num() > 0;
	}

	/**
	 * Checks if any coder can be created to match this combination of resource and configuration.
	 *
	 * @tparam TResource Type of AVResource the coder will support.
	 * @tparam TConfig Type of AVConfig the coder will support.
	 * @param NewDevice Device to filter the coder against, and to open it with.
	 * @param NewInstance Instance to filter the coder against, and to open it with.
	 * @return True if a coder could be created.
	 */
	template <typename TResource, typename TConfig>
	static bool IsSupported(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
	{
		for (TFactory<TResource, TConfig> const& Factory : GetFactories<TResource, TConfig>())
		{
			if (Factory.IsCompatible(NewDevice, NewInstance))
			{
				return true;
			}
		}

		return false;
	}

	/**
	 * Create a type-safe coder if possible, open and ready for use.
	 *
	 * @tparam TResource Type of AVResource the coder will support.
	 * @tparam TConfig Type of AVConfig the coder will support.
	 * @param NewDevice Device to filter the coder against, and to open it with.
	 * @param NewInstance Instance to filter the coder against, and to open it with.
	 * @return The constructed and opened coder, null if one could not be created.
	 */
	template <typename TResource, typename TConfig>
	static TSharedPtr<TDomain<TResource, TConfig>> Create(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
	{
		TArray<TFactory<TResource, TConfig>>& TypedFactories = GetFactories<TResource, TConfig>();
		if (TypedFactories.Num() <= 0)
		{
			FAVResult::Log(EAVResult::ErrorCreating, TEXT("Factory list is empty, vendor plugins that provide coder implementations (such as NVCodec) are likely not enabled"));
		}
		
		for (TFactory<TResource, TConfig> const& Factory : TypedFactories)
		{
			if (Factory.IsCompatible(NewDevice, NewInstance))
			{
				TSharedPtr<TDomain<TResource, TConfig>> const Output = Factory.Create();
				if (Output.IsValid() && Output->Open(NewDevice, NewInstance))
				{
					return Output;
				}
			}
		}

		return nullptr;
	}

	/**
	 * Create a type-safe coder if possible, open and ready for use.
	 *
	 * @tparam TResource Type of AVResource the coder will support.
	 * @tparam TConfig Type of AVConfig the coder will support.
	 * @param NewDevice Device to filter the coder against, and to open it with.
	 * @param NewConfig Config to filter the coder against, and to open it with.
	 * @return The constructed and opened coder, null if one could not be created.
	 */
	template <typename TResource, typename TConfig>
	static TSharedPtr<TDomain<TResource, TConfig>> Create(TSharedRef<FAVDevice> const& NewDevice, TConfig const& NewConfig)
	{
		TSharedRef<FAVInstance> const NewInstance = MakeShared<FAVInstance>();
		NewInstance->Set(NewConfig);

		return Create<TResource, TConfig>(NewDevice, NewInstance);
	}

	/**
	 * Create a type-safe coder, open and ready for use.
	 *
	 * @tparam TResource Type of AVResource the coder will support.
	 * @tparam TConfig Type of AVConfig the coder will support.
	 * @param NewDevice Device to filter the coder against, and to open it with.
	 * @param NewConfig Config to filter the coder against, and to open it with.
	 * @return The constructed and opened coder. Throws an exception if one could not be created.
	 */
	template <typename TResource, typename TConfig>
	static TSharedRef<TDomain<TResource, TConfig>> CreateChecked(TSharedRef<FAVDevice> const& NewDevice, TConfig const& NewConfig)
	{
		TSharedPtr<TDomain<TResource, TConfig>> const Output = Create<TResource, TConfig>(NewDevice, NewConfig);

		verifyf(Output.IsValid(), TEXT("No coder could be created"));

		return Output.ToSharedRef();
	}

	/**
	 * Create a type-safe coder, open and ready for use.
	 * NOTE: This is for advanced use cases only (ie. sharing an instance between multiple coders). Prefer the above version of this function if possible.
	 *
	 * @tparam TResource Type of AVResource the coder will support.
	 * @tparam TConfig Type of AVConfig the coder will support.
	 * @param NewDevice Device to filter the coder against, and to open it with.
	 * @param NewInstance Instance to filter the coder against, and to open it with.
	 * @return The constructed and opened coder. Throws an exception if one could not be created.
	 */
	template <typename TResource, typename TConfig>
	static TSharedRef<TDomain<TResource, TConfig>> CreateChecked(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance)
	{
		TSharedPtr<TDomain<TResource, TConfig>> const Output = Create<TResource, TConfig>(NewDevice, NewInstance);

		verifyf(Output.IsValid(), TEXT("No coder could be created"));

		return Output.ToSharedRef();
	}

private:
	/**
	 * The physical device this coder runs on.
	 */
	TSharedPtr<FAVDevice> Device;
	
	/**
	 * The shared data cache of this coder.
	 */
	TSharedPtr<FAVInstance> Instance;

public:
	/**
	 * @return Get the physical device this coder runs on.
	 */
	virtual TSharedPtr<FAVDevice> const& GetDevice() const override
	{
		return this->Device;
	}
	
	/**
	 * @return Get the shared data cache of this coder.
	 */
	virtual TSharedPtr<FAVInstance> const& GetInstance() const override
	{
		return Instance;
	}

	/**
	 * Construct the underlying coder architecture on a physical device with a configuration.
	 * 
	 * @param NewDevice The physical device on which to construct this coder.
	 * @param NewInstance The cache with which to construct this coder.
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override
	{
		this->Device = NewDevice;
		this->Instance = NewInstance;

		return EAVResult::Success;
	}
};

#undef UE_API
