// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVInstance.h"

#include "AVConfig.generated.h"

/**
 * Simple presets for whole AV pipeline.
 */
UENUM()
enum class EAVPreset : uint8
{
	UltraLowQuality,
	LowQuality,
	Default,
	HighQuality,
	Lossless,
};

/**
 * Latency mode for the AV pipeline.
 */
UENUM()
enum class EAVLatencyMode : uint8
{
	UltraLowLatency,
	LowLatency,
	Default,
};

/**
 * Base struct for codec configuration.
 */
struct FAVConfig
{
public:
	EAVPreset	   Preset;
	EAVLatencyMode LatencyMode;

	FAVConfig(EAVPreset Preset = EAVPreset::Default)
		: Preset(Preset)
		, LatencyMode(EAVLatencyMode::Default)
	{
	}
};
