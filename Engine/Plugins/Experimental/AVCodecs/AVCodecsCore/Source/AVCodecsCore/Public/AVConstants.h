// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Collections of constants used throughout AVCodecs
 */
namespace Video
{
	constexpr int MaxSpatialLayers = 5;
	constexpr int MaxTemporalStreams = 4;
	constexpr int MaxSimulcastStreams = 3;
} // namespace Video