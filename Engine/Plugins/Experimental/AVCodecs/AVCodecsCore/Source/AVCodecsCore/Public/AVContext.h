// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Base class for device contexts, for use by resources and platform vendor architecture
 */
class FAVContext
{
public:
	virtual ~FAVContext() = default;
};
