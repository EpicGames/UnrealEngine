// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVContext.h"
#include "AVUtility.h"

#define UE_API AVCODECSCORE_API

/**
 * A representation of a physical computer device, be it a GPU or a CPU or an external device, that can hold device resource contexts (Vulkan, D3D, CUDA, etc).
 */
class FAVDevice
{
private:
	/*
	 * Type-erased list of contexts on this device
	 */
	TTypeMap<FAVContext> Contexts;

public:
	/**
	 * Get the global hardware device.
	 *
	 * @param Index Index of the device if there are multiple hardware devices present (multiple GPUs or external cards).
	 * @return The global hardware device at Index.
	 */
	static UE_API TSharedRef<FAVDevice>& GetHardwareDevice(int32 Index = 0);

	/**
	 * Get the global software device.
	 *
	 * @return The global software device.
	 */
	static UE_API TSharedRef<FAVDevice>& GetSoftwareDevice();

	/**
	 * Check if this device has a specific context.
	 *
	 * @tparam TContext Type of context to check for.
	 * @return Whether this device has a context of that type.
	 */
	template <typename TContext>
	bool HasContext() const
	{
		return Contexts.Contains<TContext>();
	}

	/**
	 * Get a typed context from this device, if it exists.
	 *
	 * @tparam TContext Type of context to get.
	 * @return The context if it exists, an invalid pointer if it does not.
	 */
	template <typename TContext>
	TSharedPtr<TContext> const& GetContext() const
	{
		return Contexts.Get<TContext>();
	}

	/**
	 * Set a context by type on this device.
	 *
	 * @tparam TContext Type of context to set.
	 * @param NewContext Context to set.
	 */
	template <typename TContext>
	void SetContext(TSharedPtr<TContext> const& NewContext)
	{
		Contexts.Set<TContext>(NewContext);
	}

	FAVDevice() = default;
	virtual ~FAVDevice() = default;
};

#undef UE_API
