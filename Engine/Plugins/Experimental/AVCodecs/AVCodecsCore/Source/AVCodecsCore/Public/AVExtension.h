// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "CoreMinimal.h"
#include "Templates/UnrealTypeTraits.h"

#include "AVResult.h"

/**
 * HOW TO USE
 *
 * Allows compile-time extension of types without directly modifying them across modules.
 *
 * 1) Like any static function, declare (by specialization) the extension in a header:
 * 
 * template <>
 * FAVResult FAVExtension::TransformConfig(FVideoConfigH264& OutConfig, FVideoConfigBitstream const& InConfig);
 *
 * 2) Then define the extension in the source:
 * 
 * template <>
 * DLLEXPORT FAVResult FAVExtension::TransformConfig(FVideoConfigH264& OutConfig, FVideoConfigBitstream const& InConfig)
 * {
 *		// do stuff
 * }
 *
 * Be sure to EXPORT the definition and it will be automatically linked by consumers.
 */

/**
 * Static extension system for simple composition of different resource/config types, via exported specializations
 */
struct FAVExtension
{
public:
	/**
	 * Check if a type is compatible in a certain context, with a certain device. Very general use.
	 * Returns true by default, so specialize this to specifically disallow/test certain combinations rather than enable them.
	 *
	 * Usage: TAVCoder factory for AVResource usage, ie. FAVExtension::IsCompatible<FVideoEncoderNVENC, FVideoResourceD3D12>(Device); 
	 *
	 * @tparam TContext Context against which to check compatability.
	 * @tparam TOption Type to check for compatability.
	 * @param Device The device associated with the type.
	 * @return True if the type is compatible in this context.
	 */
	template <typename TContext, typename TOption>
	static bool IsCompatible(TSharedPtr<class FAVDevice> const& Device) { return true; }

	/**
	 * Check if a type is compatible in a certain context, with a certain instance. Very general use.
	 * Returns true by default, so specialize this to specifically disallow/test certain combinations rather than enable them.
	 *
	 * Usage: TAVCoder factory for AVConfig usage, ie. FAVExtension::IsCompatible<FVideoEncoderNVENC, FVideoConfigH264>(Instance); 
	 *
	 * @tparam TContext Context against which to check compatability.
	 * @tparam TOption Type to check for compatability.
	 * @param Instance The instance associated with the type.
	 * @return True if the type is compatible in this context.
	 */
	template <typename TContext, typename TOption>
	static bool IsCompatible(TSharedPtr<class FAVInstance> const& Instance) { return true; }

	/**
	 * Transform one resource type into another. Implementation is deleted, so it must be explicitly specialized to exist.
	 *
	 * @tparam TOutput Resource type to convert to.
	 * @tparam TInput Resource type to convert from.
	 * @param OutResource The resource to write to.
	 * @param InResource The resource to read from.
	 * @return Result of the operation, @see FAVResult.
	 */
	// Shortcut case for above for when the input and output types match. This does NOT duplicate the resource.
	template <typename TOutput, typename TInput>
	static FAVResult TransformResource(TSharedPtr<TOutput>& OutResource, TSharedPtr<TInput> const& InResource)
	{
		static_assert(std::is_base_of_v<TOutput, TInput>, "Input resource must be inherited from Output resource");

		OutResource = InResource;

		return EAVResult::Success;
	}

	/**
	 * Transform one configuration type into another. Naive implementation is provided that uses the assignment operator, but it can be explicitly specialized.
	 *
	 * @tparam OutputType Config type to convert to.
	 * @tparam InputType Config type to convert from.
	 * @param OutConfig The config to write to.
	 * @param InConfig The config to read from.
	 * @return Result of the operation, @see FAVResult.
	 */
	template <typename OutputType, typename InputType>
	static FAVResult TransformConfig(OutputType& OutConfig, InputType const& InConfig)
	{
		if constexpr (std::is_same_v<InputType, OutputType> || !std::is_base_of_v<InputType, OutputType>)
		{
			OutConfig = InConfig;
		}
		else
		{
			static_cast<InputType&>(OutConfig) = InConfig;
		}

		return EAVResult::Success;
	}

	FAVExtension() = delete;
};
