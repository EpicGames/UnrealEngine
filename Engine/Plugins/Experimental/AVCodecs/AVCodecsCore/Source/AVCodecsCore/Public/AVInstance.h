// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVResult.h"
#include "AVUtility.h"

/**
 * The shared data and logic of a single logical pipeline, usually one or more coders.
 */
class FAVInstance
{
private:
	/*
	 * Type-erased list of data on this instance
	 */
	TTypeMap<void> Data;

public:
	/**
	 * Check if this instance has a specific type of data.
	 *
	 * @tparam TData Type of data to check for.
	 * @return Whether this instance has data of that type.
	 */
	template <typename TData>
	bool Has() const
	{
		return Data.Contains<TData>();
	}

	/**
	 * Add default data by type to this instance.
	 *
	 * @tparam TData Type of data to add.
	 */
	template <typename TData>
	void Add()
	{
		Edit<TData>();
	}

	/**
	 * Check if this instance has a specific type of data, and return it if it exists.
	 *
	 * @tparam TData Type of data to get.
	 * @param OutData A copy of the data, if it exists.
	 * @return Whether the instance has data of the type.
	 */
	template <typename TData>
	bool TryGet(TData& OutData)
	{
		TSharedPtr<TData>& Result = Data.Edit<TData>();
		if (Result.IsValid())
		{
			OutData = *Result;

			return true;
		}

		return false;
	}

	/**
	 * Get typed data from this instance. Will create default data of this type if it does not already exist.
	 *
	 * @tparam TData Type of data to get.
	 * @return The typed data.
	 */
	template <typename TData>
	TData const& Get()
	{
		return Edit<TData>();
	}

	/**
	 * Set data by type on this instance.
	 *
	 * @tparam TData Type of data to set.
	 * @param NewData Data to set.
	 */
	template <typename TData>
	void Set(TData const& NewData)
	{
		Edit<TData>() = NewData;
	}

	/**
	 * Check if this instance has a specific type of data, and return a mutable reference to it if it exists.
	 *
	 * @tparam TData Type of data to get.
	 * @return A pointer to the data, if it exists. Not valid outside of the scope from which this is called.
	 */
	template <typename TData>
	TData* TryEdit()
	{
		TSharedPtr<TData>& Result = Data.Edit<TData>();
		if (Result.IsValid())
		{
			return Result.Get();
		}

		return nullptr;
	}

	/**
	 * Get a mutable reference to typed data held on this instance. Will create default data of this type if it does not already exist.
	 *
	 * @tparam TData Type of data to get.
	 * @return The typed data.
	 */
	template <typename TData>
	TData& Edit()
	{
		TSharedPtr<TData>& Result = Data.Edit<TData>();
		if (!Result.IsValid())
		{
			Result = MakeShared<TData>();
		}
		
		return *Result;
	}

	/**
	 * Remove a specific type of data from this instance.
	 *
	 * @tparam TData Type of data to remote.
	 */
	template <typename TData>
	void Remove()
	{
		Data.Remove<TData>();
	}

	FAVInstance() = default;
	virtual ~FAVInstance() = default;
};
