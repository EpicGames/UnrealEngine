// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

/**
 * This struct defines the memory footprint of a block of data.
 */
struct FAVLayout
{
public:
	uint32 Stride = 0;
	uint32 Offset = 0;
	uint32 Size = 0;

	FAVLayout() = default;
	FAVLayout(uint32 Stride, uint32 Offset, uint32 Size)
		: Stride(Stride)
		, Offset(Offset)
		, Size(Size)
	{
	}

	bool operator==(FAVLayout const& RHS) const
    {
		return Stride == RHS.Stride && Offset == RHS.Offset && Size == RHS.Size;
    }

	bool operator!=(FAVLayout const& RHS) const
	{
		return !(*this == RHS);
	}
};
