// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "HAL/FileManager.h"
#include "Templates/SharedPointer.h"

/**
 * Base container for raw encoded AV resources, created by encoders and consumed by decoders.
 * Can be treated like a networking packet, where the encoder and decoder are separate machines and communicate only through this medium.
 */
struct FAVPacket
{
	/**
	 * Safe pointer to packet data.
	 */
	TSharedPtr<uint8> DataPtr;

	/**
	 * Size of packet data.
	 */
	uint64 DataSize;

	/**
	 * Timestamp of packet.
	 */
	uint64 Timestamp;

	/**
	 * Packet index in the sequence.
	 */
	uint64 Index;

	/**
	 * Convenience wrapper to treat raw data as an array view.
	 *
	 * @return Raw data as an array view.
	 */
	inline TArrayView64<uint8> GetData() const { return TArrayView64<uint8>(DataPtr.Get(), DataSize); }

	inline void WriteToFile(TCHAR* FileName) const {
		FString SaveName = FString::Printf(TEXT("%s%s"), *FPaths::ProjectSavedDir(), FileName);

		UE_LOGF(LogTemp, Display, "Saving Packet to %ls", *SaveName);
		FFileHelper::SaveArrayToFile(TArrayView64<uint8>(DataPtr.Get(), DataSize), *SaveName, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
	}

	/**
	 * Test whether raw data is empty or not set.
	 *
	 * @return True if raw data is empty or not set.
	 */
	bool IsEmpty() const
	{
		return !DataPtr.IsValid() || DataSize == 0;
	}

	FAVPacket() = default;
	FAVPacket(TSharedPtr<uint8> const& DataPtr, uint64 DataSize, uint64 Timestamp, uint64 Index)
		: DataPtr(DataPtr)
		, DataSize(DataSize)
		, Timestamp(Timestamp)
		, Index(Index)
	{
	}
};
