// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVDevice.h"
#include "AVLayout.h"
#include "AVResult.h"

#define UE_API AVCODECSCORE_API

/**
 * Base wrapper for an immutable device resource. Bundles necessary data that may not be easily retrieved on without an RHI, such as size on device.
 * If the underlying device resource must change, this resource wrapper must be recreated.
 */
class FAVResource
{
private:
	/**
	 * Reference to parent device. Must exist.
	 */
	TSharedRef<FAVDevice> Device;

	/**
	 * Layout of data in device memory.
	 */
	FAVLayout Layout;

	/**
	 * Mutex used for resource synchronization.
	 */
	FCriticalSection Mutex;

	/**
	 * All mappings of this resource into other resources (or other objects), ensuring appropriate cleanup when the root resource is destroyed.
	 * Stored as a type-erased list of shared pointers.
	 */
	TTypeMap<void> Mappings;

public:
	/**
	 * @return Get a reference to our device parent. Is always valid.
	 */
	inline TSharedRef<FAVDevice> const& GetDevice() const { return Device; }

	/**
	 * @return Get the layout of our data in device memory.
	 */
	inline FAVLayout const& GetLayout() const { return Layout; }

	/**
	 * @return Get the offset of our data in device memory.
	 */
	inline uint64 GetOffset() const { return Layout.Offset; }

	/**
	 * @return Get the size of our data in device memory.
	 */
	inline uint64 GetMemorySize() const { return Layout.Size; }
	
	/**
	 * @return Get the stride of our data in device memory.
	 */
	inline uint64 GetStride() const { return Layout.Stride; }

	UE_API FAVResource(TSharedRef<FAVDevice> const& Device, FAVLayout const& Layout);
	virtual ~FAVResource() = default;

	/**
	 * Test if the underlying resource is usable or misconfigured.
	 *
	 * @return Result of the operation, @see FAVResult.
	 */
	UE_API virtual FAVResult Validate() const;

	/**
	 * Lock this resource for internal use. Is a blocking call until the lock can be taken.
	 */
	UE_API virtual void Lock();
	
	/**
	 * Lock this resource for internal use, within the current scope. Is a blocking call until the lock can be taken.
	 *
	 * @return The lock object. Capture this, because when it goes out of scope it will unlock the resource.
	 */
	UE_API virtual FScopeLock LockScope();

	/**
	 * Release this resource for external use.
	 */
	UE_API virtual void Unlock();

	/**
	 * Create a mapping of this resource to another resource (or other object). Returned reference must then be set by the caller.
	 *
	 * @tparam TMapping Object type to map this resource to.
	 * @return Shared pointer reference to the mapping, initially invalid until set by the caller.
	 */
	template <typename TMapping>
	TSharedPtr<TMapping>& PinMapping()
	{
		return Mappings.Edit<TMapping>();
	}
};

#undef UE_API
