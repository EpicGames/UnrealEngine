// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Math/NumericLimits.h"
#include "HAL/Platform.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAVCodecs, Log, All);

/**
 * HOW TO USE
 *
 * Simple multi-level result wrapper. Allows consumers to be as specific as they want (or can be), but forces them to always return something.
 *
 * 1) Lowest level is a result code:
 *
 * return EAVResult::Success;
 *
 * 2) Middle level is a wrapped struct:
 *
 * return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder is not open"));
 * return FAVResult(EAVResult::ErrorCreating, TEXT("Could not create encoder"), TEXT("NVENC"), NvencErrorCode);
 *
 * 3) Highest level allows optional simultaneous return of a value.
 *
 *	TAVResult<FVideoPacket> ReceivePacket()
 *	{
 *		// Success
 *		return Packet;
 *
 *		// Fail
 *		return FAVResult(EAVResult::PendingInput, TEXT("Not enough data to construct a packet"));
 *	}
 */

/**
 * Low level AV result. Use specific result codes where possible for better logging.
 * Result codes are laid out in 'ranges', ie. ErrorCreating/ErrorResolving/ErrorUnlocking are all in the Error range.
 *
 * MAINTAINER NOTE: When adding a new result value, add a matching case statement to ToString(EAVResult).
 */
enum class EAVResult : uint16
{
	Unknown = 0,
	
	Fatal = 1000,
	FatalUnsupported,
	
	Error = 2000,
	ErrorUnsupported,
	ErrorInvalidState,
	ErrorCreating,
	ErrorDestroying,
	ErrorResolving,
	ErrorMapping,
	ErrorUnmapping,
	ErrorLocking,
	ErrorUnlocking,
	
	Warning = 3000,
	WarningInvalidState,

	Pending = 4000,
	PendingInput,
	PendingOutput,
	
	Success = 5000,
};

/**
 * AVResult string parser.
 *
 * MAINTAINER NOTE: When adding a new EAVResult value, add a matching case statement to this.
 */
AVCODECSCORE_API FString ToString(EAVResult Result);

/**
 * Primary AV result type. Contains a result code, an optional message, and optional specific data about the platform vendor where the error occured.
 * Automatically logs on warning and lower if not handled.
 */
struct FAVResult
{
private:
	static int32 constexpr EMPTY_VENDOR_VALUE = TNumericLimits<int32>::Min();

public:
	/**
	 * Statically log a result without constructing and returning it.
	 *
	 * @param Value Low level AV result code.
	 * @param Message Optional message to go along with the result code.
	 * @param Vendor Optional name of vendor where this error occured.
	 * @param VendorValue Optional result code passed to us by the vendor.
	 */
	inline static void Log(EAVResult Value, FString const& Message = "", FString const& Vendor = "", int32 VendorValue = EMPTY_VENDOR_VALUE)
	{
		FAVResult(Value, Message, Vendor, VendorValue).Log();
	}

	/**
	 * AV result code.
	 */
	EAVResult Value = EAVResult::Unknown;

	/**
	 * Optional message alongside AV result code.
	 */
	FString Message = "";

	/**
	 * Optional vendor where this error occured.
	 */
	FString Vendor = "";

	/**
	 * Optional result code supplied by the vendor.
	 */
	int32 VendorValue = EMPTY_VENDOR_VALUE;

	/**
	 * Bitflag for whether this result has been handled (or whether is still must be logged).
	 */
	uint8 bHandled : 1;

	FAVResult() = default;

	/**
	 * @param Value Low level AV result code.
	 * @param Message Optional message to go along with the result code.
	 * @param Vendor Optional name of vendor where this error occured.
	 * @param VendorValue Optional result code passed to us by the vendor.
	 */
	FAVResult(EAVResult Value, FString const& Message = "", FString const& Vendor = "", int32 VendorValue = EMPTY_VENDOR_VALUE)
		: Value(Value)
		, Message(Message)
		, Vendor(Vendor)
		, VendorValue(VendorValue)
		, bHandled(false)
	{
	}

	FAVResult(FAVResult const& From)
		: Value(From.Value)
		, Message(From.Message)
		, Vendor(From.Vendor)
		, VendorValue(From.VendorValue)
		, bHandled(From.bHandled)
	{
		const_cast<FAVResult&>(From).bHandled = true;
	}

	~FAVResult()
	{
		// Automatically log this result if it was unhandled
		if (!bHandled && Value < EAVResult::Pending)
		{
			Log();
		}
	}

	inline FAVResult& operator=(FAVResult const& From)
	{
		Value = From.Value;
		Message = From.Message;
		Vendor = From.Vendor;
		VendorValue = From.VendorValue;
		bHandled = From.bHandled;

		const_cast<FAVResult&>(From).bHandled = true;

		return *this;
	}

	inline bool operator==(EAVResult OtherValue) const
	{
		return Value == OtherValue;
	}

	inline bool operator!=(EAVResult OtherValue) const
	{
		return !(*this == OtherValue);
	}

	inline operator bool() const
	{
		return IsSuccess();
	}

	/**
	 * @return True if AV result code is in the range of Fatal.
	 */
	inline bool IsFatal() const
	{
		return Value >= EAVResult::Fatal && Value < EAVResult::Error;
	}

	/**
	 * @return True if AV result code is NOT in the range of Fatal.
	 */
	inline bool IsNotFatal() const
	{
		return !IsFatal();
	}

	/**
	 * @return True if AV result code is in the range of Error.
	 */
	inline bool IsError() const
	{
		return Value >= EAVResult::Error && Value < EAVResult::Warning;
	}

	/**
	 * @return True if AV result code is NOT in the range of Error.
	 */
	inline bool IsNotError() const
	{
		return !IsError();
	}

	/**
	 * @return True if AV result code is in the range of Warning.
	 */
	inline bool IsWarning() const
	{
		return Value >= EAVResult::Warning && Value < EAVResult::Pending;
	}

	/**
	 * @return True if AV result code is NOT in the range of Warning.
	 */
	inline bool IsNotWarning() const
	{
		return !IsWarning();
	}

	/**
	 * @return True if AV result code is in the range of Pending.
	 */
	inline bool IsPending() const
	{
		return Value >= EAVResult::Pending;
	}

	/**
	 * @return True if AV result code is NOT in the range of Pending.
	 */
	inline bool IsNotPending() const
	{
		return !IsPending();
	}

	/**
	 * @return True if AV result code is in the range of Success.
	 */
	inline bool IsSuccess() const
	{
		return Value >= EAVResult::Success;
	}

	/**
	 * @return True if AV result code is NOT in the range of Success.
	 */
	inline bool IsNotSuccess() const
	{
		return !IsSuccess();
	}

	/**
     * Flag this result as handled, so that it won't be logged.
     */
    AVCODECSCORE_API FAVResult& Handle();

	/**
	 * Convert this error into a readable string.
	 *
	 * @return Pretty string.
	 */
	AVCODECSCORE_API FString ToString() const;

	/**
	 * Logs this result to the internal AVCodecCore log category.
	 */
	AVCODECSCORE_API void Log() const;
};

/**
 * Templated AV result type that allows returning a result code and an optional value. Functions identically to FAVResult.
 */
template <typename TReturnValue>
struct TAVResult : public FAVResult
{
public:
	/**
	 * Optional return value, is only valid when AV result code is in the range of Success.
	 */
	TReturnValue ReturnValue;

	TAVResult() = default;

	TAVResult(TReturnValue const& ReturnValue)
		: FAVResult(EAVResult::Success)
		, ReturnValue(ReturnValue)
	{
	}

	TAVResult(FAVResult const& Result)
		: FAVResult(Result)
		, ReturnValue(TReturnValue())
	{
	}

	TAVResult(TReturnValue const& ReturnValue, FAVResult const& Result)
		: FAVResult(Result)
		, ReturnValue(ReturnValue)
	{
	}

	TAVResult(EAVResult Value, FString const& Message = "", FString const& Vendor = "", int32 VendorValue = -1)
		: FAVResult(Value, Message, Vendor, VendorValue)
		, ReturnValue(TReturnValue())
	{
	}

	TAVResult(TReturnValue const& ReturnValue, EAVResult Value, FString const& Message = "", FString const& Vendor = "", int32 VendorValue = -1)
		: FAVResult(Value, Message, Vendor, VendorValue)
		, ReturnValue(ReturnValue)
	{
	}

	inline operator TReturnValue&()
	{
		return ReturnValue;
	}

	inline operator TReturnValue const&() const
	{
		return ReturnValue;
	}
};
