// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API AVCODECSCORE_API

/**
 * All generic (non-AV) utility types.
 */

/**
 * Base type for a deferred object, to be resolved by an external application with specific and unpredictable requirements.
 *
 * @tparam TObject Type of deferred object.
 * @tparam TArgs Argument list for deferred Resolve function.
 */
template <typename TObject, typename... TArgs>
class TResolvable
{
private:
	/**
	 * Stored args used to resolve this resource, to avoid re-resolving the same object.
	 * Only valid when a deferred object has been resolved.
	 */
	TTuple<TArgs...> ResolvedArgs;

	/**
	 * The possibly-resolved deferred object.
	 */
	TSharedPtr<TObject> Resolved;
	
public:
	TResolvable() = default;
	virtual ~TResolvable() = default;

	TObject* Get() const
	{
		return Resolved.Get();
	}

	TObject* operator->() const
	{
		return Resolved.Get();
	}

	operator TSharedPtr<TObject>&()
	{
		return Resolved;
	}

	operator TSharedPtr<TObject> const&() const
	{
		return Resolved;
	}

	bool operator==(TResolvable<TObject, TArgs...> const& Other) const
	{
		return Resolved == Other.Resolved;
	}
	
	bool operator!=(TResolvable<TObject, TArgs...> const& Other) const
	{
		return !(*this == Other);
	}

	TSharedRef<TObject> ToSharedRef() const
	{
		return Resolved.ToSharedRef();
	}

	/**
	 * Test whether the deferred object has been successfully resolved with specific arguments.
	 *
	 * @param Args Arguments that the deferred object must have been resolved with.
	 * @return True if the deferred object is valid.
	 */
	bool IsResolved(TArgs const&... Args) const
	{
		if (Resolved.IsValid() && ResolvedArgs == MakeTuple(Args...))
		{
			return true;
		}

		return false;
	}

	/**
	 * Test whether the deferred object has been successfully resolved.
	 *
	 * @return True if the deferred object is valid.
	 */
	bool IsResolved() const
	{
		return ResolvedArgs.ApplyAfter([this](TArgs const&... Args) { return IsResolved(Args...); });
	}

	/**
	 * Attempt to resolve the deferred object with specific arguments.
	 *
	 * @param Args Arguments that the deferred object must be resolved with.
	 * @return True if the deferred object is successfully resolved.
	 */
	bool Resolve(TArgs const&... Args)
	{
		if (IsResolved(Args...))
		{
			return true;
		}

		ResolvedArgs = MakeTuple(Args...);
		Resolved = TryResolve(Args...);

		return Resolved.IsValid();
	}

	/**
	 * Clear the resolved deferred object.
	 */
	virtual void Reset()
	{
		if (Resolved.IsValid())
		{
			Resolved.Reset();
		}
	}
	
protected:
	/**
	 * Deferred resolve, to be overridden by a variety of implementations.
	 *
	 * @param Args Arguments that the deferred object must be resolved with.
	 * @return Resolved deferred object if successful, invalid if not.
	 */
	virtual TSharedPtr<TObject> TryResolve(TArgs const&... Args) = 0;
};

/**
 * Delegate-based implementation of TResolvable.
 *
 * @tparam TObject Type of deferred object.
 * @tparam TArgs Argument list for deferred Resolve function.
 */
template <typename TObject, typename... TArgs>
class TDelegated : public TResolvable<TObject, TArgs...>
{
public:
	/**
	 * Delegate definition to be used to resolve the deferred object.
	 */
	typedef TFunction<void(TSharedPtr<TObject>&, TArgs...)> FDelegate;
	FDelegate const Delegate;

	TDelegated(FDelegate const& Delegate)
		: Delegate(Delegate)
	{
	}

protected:
	virtual TSharedPtr<TObject> TryResolve(TArgs const&... Args) override
	{
		TSharedPtr<TObject> OldResolved = *this;
		Delegate(OldResolved, Args...);

		return OldResolved;
	}
};

/**
 * Macro to register a type to be usable with FTypeID.
 */
#define REGISTER_TYPEID(Type) \
	template <> DLLEXPORT FTypeID FTypeID::Get<Type>() { static char Ref; return &Ref; }

#define DECLARE_TYPEID(Type, ExportMacro) \
	template <> ExportMacro FTypeID FTypeID::Get<Type>();

/**
 * A simple type-erased type identifier, without the use of RTTI or the unreal type system.
 */
struct FTypeID
{
private:
	/**
	 * Pointer to raw type identifier. Identifier is static so no cleanup.
	 */
	void* Raw;

	// Private, require all FTypeID's to be valid.
	FTypeID(void* Raw)
		: Raw(Raw)
	{
	}

public:
	/**
	 * Get the static ID of a type.
	 *
	 * @tparam TType Type to identify. Must be exported by the REGISTER_TYPEID macro above or this will fail with a linker error.
	 */
	template <typename TType>
	static AVCODECSCORE_API FTypeID Get();

	FTypeID() = default;

	bool operator==(FTypeID const& Other) const
	{
		return Raw != nullptr && Raw == Other.Raw;
	}

	bool operator!=(FTypeID const& Other) const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(FTypeID const& Key)
	{
		return ::GetTypeHash(Key.Raw);
	}
};

/**
 * Typed map of objects that are uniquely indexed by their type.
 */
template <typename TBaseType>
class TTypeMap
{
private:
	/**
	 * Type-erased list of objects, stored by their base type.
	 */
	TMap<FTypeID, TSharedPtr<TBaseType>> Objects;

public:
	TTypeMap() = default;

	/**
	 * Tests whether this map contains an object of a specific type.
	 *
	 * @tparam TType Type of object to test.
	 * @return True if an object of the type exists.
	 */
	template <typename TType UE_REQUIRES(std::is_convertible_v<TType*, TBaseType*>)>
	bool Contains() const
	{
		return Get<TType>().IsValid();
	}

	/**
	 * Retrieves an object of a specific type, if it already exists in the map.
	 *
	 * @tparam TType Type of object to find.
	 * @return Typed object if it exists in the map, invalid if it does not.
	 */
	template <typename TType UE_REQUIRES(std::is_convertible_v<TType*, TBaseType*>)>
	TSharedPtr<TType> const& Get() const
	{
		// Can't reinterpret cast, but we know this is safe
		if (TSharedPtr<TType>* Object = (TSharedPtr<TType>*)Objects.Find(FTypeID::Get<TType>()))
		{
			return *Object;
		}

		static TSharedPtr<TType> Empty = nullptr;
		return Empty;
	}

	/**
	 * Add an object to the map, indexed by its type. If an object of this type already exists then replace it.
	 *
	 * @tparam TType Type of object to add.
	 * @param NewObject Object to add.
	 */
	template <typename TType UE_REQUIRES(std::is_convertible_v<TType*, TBaseType*>)>
	void Set(TSharedPtr<TType> const& NewObject)
	{
		Objects.Add(FTypeID::Get<TType>(), NewObject);
	}

	/**
	 * Retrieves an object of a specific type, and if it does not exist in the map then create an invalid reference to it (which must then be set by the caller).
	 *
	 * @tparam TType Type of the object to edit.
	 * @return Shared pointer reference to the mapping, invalid if the object did not already exist.
	 */
	template <typename TType UE_REQUIRES(std::is_convertible_v<TType*, TBaseType*>)>
	TSharedPtr<TType>& Edit()
	{
		// Can't reinterpret cast, but we know this is safe
		return *(TSharedPtr<TType>*)&Objects.FindOrAdd(FTypeID::Get<TType>());
	}
	
	/**
	 * Remove an object type from the map.
	 *
	 * @tparam TType Type of object to add.
	 */
	template <typename TType UE_REQUIRES(std::is_convertible_v<TType*, TBaseType*>)>
	void Remove(TSharedPtr<TType> const& NewContext)
	{
		Objects.Remove(FTypeID::Get<TType>());
	}
};

/**
 * API abstraction with static singletons.
 */
class FAPI
{
private:
	/**
	 * Type-erased map of API singletons.
	 */
	static UE_API TTypeMap<FAPI> Singletons;

public:
	/**
	 * Gets or creates an API of a specific type.
	 *
	 * @tparam TAPI Type of API to find, which must inherit from FAPI.
	 * @return Direct reference to the typed API.
	 */
	template <typename TAPI UE_REQUIRES(std::is_base_of_v<FAPI, TAPI>)>
	static TAPI const& Get()
	{
		TSharedPtr<TAPI>& Singleton = Singletons.Edit<TAPI>();
		if (!Singleton.IsValid())
		{
			Singleton = MakeShared<TAPI>();
		}

		return *Singleton;
	}

	/**
	 * Clean up an API of a specific type.
	 *
	 * @tparam TAPI Type of the API to clean up, which must inherit from FAPI.
	 */
	template <typename TAPI UE_REQUIRES(std::is_base_of_v<FAPI, TAPI>)>
	static void Shutdown()
	{
		Singletons.Edit<TAPI>().Reset();
	}

	/**
	 * Test if the API loaded correctly, or has since been cleaned up.
	 *
	 * @return True if the API is valid.
	 */
	virtual bool IsValid() const = 0;

protected:
	FAPI() = default;
	virtual ~FAPI() = default;
};

#undef UE_API
