// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVCoder.h"
#include "Audio/AudioPacket.h"
#include "Audio/AudioResource.h"

/*
 * Implementation of Audio Decoding domain, see TAVCoder for inheritance model
 */

struct FAudioDecoderConfig : public FAVConfig
{
public:
	FAudioDecoderConfig(EAVPreset Preset = EAVPreset::Default)
		: FAVConfig(Preset) { }
};

/**
 * Audio decoder with a factory, that supports typesafe resource handling and configuration.
 *
 * @see TAVCodec
 */
template <typename TResource = void, typename TConfig = void>
class TAudioDecoder : public TAVCoder<TAudioDecoder, TResource, TConfig>
{
public:
	/**
	 * Wrapper decoder that transforms resource/config types for use with a differently typed child decoder.
	 *
	 * @tparam TChildResource Type of child resource.
	 * @tparam TChildConfig Type of child config.
	 */
	template <typename TChildResource, typename TChildConfig>
	class TWrapper : public TAVCoder<TAudioDecoder, TResource, TConfig>::template TWrapper<TChildResource, TChildConfig>
	{
	public:
		TWrapper(TSharedRef<TAudioDecoder<TChildResource, TChildConfig>> const& Child)
			: TAVCoder<TAudioDecoder, TResource, TConfig>::template TWrapper<TChildResource, TChildConfig>(Child)
		{
		}

		virtual FAVResult SendPacket(FAudioPacket const& Packet) override
		{
			if (!this->IsOpen())
			{
				return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"));
			}

			FAVResult Result = this->ApplyConfig();
			if (Result.IsNotSuccess())
			{
				return Result;
			}

			return this->Child->SendPacket(Packet);
		}

		virtual FAVResult ReceiveFrame(TResolvableAudioResource<TResource>& InOutResource) override
		{
			if (!this->IsOpen())
			{
				return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"));
			}

			TDelegatedAudioResource<TChildResource> WrappedResource([&InOutResource](TSharedPtr<TChildResource>& InOutResult, TSharedPtr<FAVDevice> const& InDevice, FAudioDescriptor const& Descriptor)
			{
				InOutResult = nullptr;

				if (InOutResource.Resolve(InDevice, Descriptor))
				{
					InOutResult = InOutResource->template PinMapping<TChildResource>();
					if (!InOutResult.IsValid() || InOutResult->Validate().IsNotSuccess())
					{
						if (FAVExtension::TransformResource<TChildResource, TResource>(InOutResult, InOutResource).IsNotSuccess())
						{
							InOutResult.Reset();
						}
					}
				}
			});

			return this->Child->ReceiveFrame(WrappedResource);
		}
	};

	/**
	 * Get generic configuration values.
	 *
	 * @return The current minimal configuration.
	 */
	virtual FAudioDecoderConfig GetMinimalConfig() override
	{
		FAudioDecoderConfig MinimalConfig;
		FAVExtension::TransformConfig<FAudioDecoderConfig, TConfig>(MinimalConfig, this->GetPendingConfig());

		return MinimalConfig;
	}

	/**
	 * Set generic configuration values.
	 *
	 * @param MinimalConfig New minimal configuration to set.
	 */
	virtual void SetMinimalConfig(FAudioDecoderConfig const& MinimalConfig) override
	{
		FAVExtension::TransformConfig<TConfig, FAudioDecoderConfig>(this->EditPendingConfig(), MinimalConfig);
	}
};

/**
 * Audio decoder with a factory, that supports typesafe resource handling.
 *
 * @see TAVCodec
 */
template <typename TResource>
class TAudioDecoder<TResource> : public TAVCoder<TAudioDecoder, TResource>
{
public:
	/**
	 * Read a finished frame out of the codec.
	 *
	 * @param InOutResource Deferred resource that can be resolved by the decoder in a desired format/size/etc. Passed by reference inout. @see TResolvableAudioResource
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult ReceiveFrame(TResolvableAudioResource<TResource>& InOutResource) = 0;
};

/**
 * Audio decoder with a factory.
 *
 * @see TAVCodec
 */
template <>
class TAudioDecoder<> : public TAVCoder<TAudioDecoder>
{
public:
	/**
	 * Send a packet to the underlying codec architecture.
	 *
	 * @param Packet Packet holding the frame data. An empty packet will perform a flush (@see FlushPackets) and invalidate the underlying architecture.
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult SendPacket(FAudioPacket const& Packet) = 0;

	/**
	 * Flush remaining frames and invalidate the underlying architecture.
	 * 
	 * @return Result of the operation, @see FAVResult.
	 */
	FAVResult FlushFrames()
	{
		return SendPacket(FAudioPacket());
	}

	/**
	 * Get generic configuration values.
	 *
	 * @return The current minimal configuration.
	 */
	virtual FAudioDecoderConfig GetMinimalConfig() = 0;

	/**
	 * Set generic configuration values.
	 *
	 * @param MinimalConfig New minimal configuration to set.
	 */
	virtual void SetMinimalConfig(FAudioDecoderConfig const& MinimalConfig) = 0;
};

typedef TAudioDecoder<> FAudioDecoder;

DECLARE_TYPEID(FAudioDecoder, AVCODECSCORE_API);
