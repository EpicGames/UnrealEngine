// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVCoder.h"
#include "Audio/AudioPacket.h"
#include "Audio/AudioResource.h"

/*
 * Implementation of Audio Encoding domain, see TAVCoder for inheritance model
 */

struct FAudioEncoderConfig : public FAVConfig
{
public:
	int32 Bitrate = 12000;
	int32 Samplerate = 44100;
	int32 NumChannels = 2;
	
	FAudioEncoderConfig(EAVPreset Preset = EAVPreset::Default)
		: FAVConfig(Preset)
	{
		switch (Preset)
		{
		case EAVPreset::UltraLowQuality:
			Bitrate = 12000 * 8;

			break;
		case EAVPreset::LowQuality:
			Bitrate = 16000 * 8;

			break;
		case EAVPreset::Default:
			Bitrate = 20000 * 8;

			break;
		case EAVPreset::HighQuality:
			Bitrate = 24000 * 8;

			break;
		case EAVPreset::Lossless:
			Bitrate = 28000 * 8;

			break;
		}
	}
};

/**
 * Audio encoder with a factory, that supports typesafe resource handling and configuration.
 *
 * @see TAVCodec
 */
template <typename TResource = void, typename TConfig = void>
class TAudioEncoder : public TAVCoder<TAudioEncoder, TResource, TConfig>
{
public:
	/**
	 * Wrapper encoder that transforms resource/config types for use with a differently typed child encoder.
	 *
	 * @tparam TChildResource Type of child resource.
	 * @tparam TChildConfig Type of child config.
	 */
	template <typename TChildResource, typename TChildConfig>
	class TWrapper : public TAVCoder<TAudioEncoder, TResource, TConfig>::template TWrapper<TChildResource, TChildConfig>
	{
	public:
		TWrapper(TSharedRef<TAudioEncoder<TChildResource, TChildConfig>> const& Child)
			: TAVCoder<TAudioEncoder, TResource, TConfig>::template TWrapper<TChildResource, TChildConfig>(Child)
		{
		}

		virtual FAVResult SendFrame(TSharedPtr<TResource> const& Resource, uint32 Timestamp) override
		{
			if (!this->IsOpen())
			{
				return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"));
			}

			FAVResult Result = this->ApplyConfig();
			if (Result.IsNotSuccess())
			{
				return Result;
			}

			if (!Resource.IsValid())
			{
				return this->Child->SendFrame(nullptr, Timestamp);
			}

			FScopeLock const Lock = Resource->LockScope();

			TSharedPtr<TChildResource>& MappedResource = Resource->template PinMapping<TChildResource>();
			if (!MappedResource.IsValid() || MappedResource->Validate().IsNotSuccess())
			{
				Result = FAVExtension::TransformResource<TChildResource, TResource>(MappedResource, Resource);
				if (Result.IsNotSuccess())
				{
					MappedResource.Reset();

					return Result;
				}
			}

			return this->Child->SendFrame(MappedResource, Timestamp);
		}

		virtual FAVResult ReceivePacket(FAudioPacket& OutPacket) override
		{
			if (!this->IsOpen())
			{
				return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"));
			}

			return this->Child->ReceivePacket(OutPacket);
		}
	};

	/**
	 * Get generic configuration values.
	 *
	 * @return The current minimal configuration.
	 */
	virtual FAudioEncoderConfig GetMinimalConfig() override
	{
		FAudioEncoderConfig MinimalConfig;
		FAVExtension::TransformConfig<FAudioEncoderConfig, TConfig>(MinimalConfig, this->GetPendingConfig());

		return MinimalConfig;
	}
	
	/**
	 * Set generic configuration values.
	 *
	 * @param MinimalConfig New minimal configuration to set.
	 */
	virtual void SetMinimalConfig(FAudioEncoderConfig const& MinimalConfig) override
	{
		FAVExtension::TransformConfig<TConfig, FAudioEncoderConfig>(this->EditPendingConfig(), MinimalConfig);
	}
};

/**
 * Audio encoder with a factory, that supports typesafe resource handling.
 *
 * @see TAVCodec
 */
template <typename TResource>
class TAudioEncoder<TResource> : public TAVCoder<TAudioEncoder, TResource>
{
public:
	/**
	 * Send a frame to the underlying codec architecture.
	 *
	 * @param Resource Resource holding the frame data. An invalid resource will perform a flush (@see FlushPackets) and invalidate the underlying architecture.
	 * @param Timestamp Recorded timestamp of the frame.
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult SendFrame(TSharedPtr<TResource> const& Resource, uint32 Timestamp) = 0;

	/**
	 * Flush remaining packets and invalidate the underlying architecture.
	 * 
	 * @return Result of the operation, @see FAVResult.
	 */
	FAVResult FlushPackets()
	{
		return SendFrame(nullptr, 0);
	}
	
	/**
	 * Flush remaining packets and invalidate the underlying architecture.
	 * 
	 * @return Result of the operation, @see FAVResult.
	 */
	FAVResult FlushAndReceivePackets(TArray<FAudioPacket>& OutPackets)
	{
		FAVResult Result = FlushPackets();
		if (Result.IsNotSuccess())
		{
			return Result;
		}

		return this->ReceivePackets(OutPackets);
	}
};

/**
 * Audio encoder with a factory.
 *
 * @see TAVCodec
 */
template <>
class TAudioEncoder<> : public TAVCoder<TAudioEncoder>
{
public:
	/**
	 * Read a finished packet out of the codec.
	 *
	 * @param OutPacket Output packet if one is complete.
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult ReceivePacket(FAudioPacket& OutPacket) = 0;

	/**
	 * Read all finished packets out of the codec.
	 *
	 * @param OutPackets Output array of completed packets.
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult ReceivePackets(TArray<FAudioPacket>& OutPackets)
	{
		FAVResult Result;

		FAudioPacket Packet;
		while ((Result = ReceivePacket(Packet)).IsSuccess())
		{
			OutPackets.Add(Packet);
		}

		return Result;
	}

	/**
	 * Get generic configuration values.
	 *
	 * @return The current minimal configuration.
	 */
	virtual FAudioEncoderConfig GetMinimalConfig() = 0;
	
	/**
	 * Set generic configuration values.
	 *
	 * @param MinimalConfig New minimal configuration to set.
	 */
	virtual void SetMinimalConfig(FAudioEncoderConfig const& MinimalConfig) = 0;
};

typedef TAudioEncoder<> FAudioEncoder;

DECLARE_TYPEID(FAudioEncoder, AVCODECSCORE_API);
