// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVPacket.h"
#include "Templates/SharedPointerFwd.h"

/**
 * Audio-specific packet container.
 */
struct FAudioPacket : public FAVPacket
{
	FAudioPacket() = default;
	FAudioPacket(TSharedPtr<uint8> const& DataPtr, uint64 DataSize, uint64 Timestamp, uint64 Index)
		: FAVPacket(DataPtr, DataSize, Timestamp, Index)
	{
	}
};
