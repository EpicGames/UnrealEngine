// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVLayout.h"
#include "AVResource.h"
#include "AVUtility.h"

#define UE_API AVCODECSCORE_API

// enum class EAudioFormat : uint8
// {
// 	// TODO (Andrew)
// };

// ?????
// inline bool operator==(EPixelFormat LHS, EAudioFormat RHS)
// {
// 	return static_cast<uint8>(LHS) == static_cast<uint8>(RHS);
// }

// inline bool operator!=(EPixelFormat LHS, EAudioFormat RHS)
// {
// 	return !(LHS == RHS);
// }

// inline bool operator==(EAudioFormat LHS, EPixelFormat RHS)
// {
// 	return static_cast<uint8>(LHS) == static_cast<uint8>(RHS);
// }

// inline bool operator!=(EAudioFormat LHS, EPixelFormat RHS)
// {
// 	return !(LHS == RHS);
// }

/**
 * This struct defines how the allocated resource memory is used in the context of audio.
 */
struct FAudioDescriptor
{
public:
	/**
	 * Number of samples in this resource.
	 */
	uint32 NumSamples;

	/**
	 * Duration of samples in this resource.
	 */
	float SampleDuration;

	FAudioDescriptor() = default;
	FAudioDescriptor(uint32 NumSamples, float SampleDuration)
		: NumSamples(NumSamples)
		, SampleDuration(SampleDuration)
	{
	}

	bool operator==(FAudioDescriptor const& RHS) const
    {
		return NumSamples == RHS.NumSamples && SampleDuration == RHS.SampleDuration;
    }

	bool operator!=(FAudioDescriptor const& RHS) const
	{
		return !(*this == RHS);
	}
};

/**
 * Base wrapper for a audio device resource.
 */
class FAudioResource : public FAVResource
{
private:
	/**
	 * Descriptor of audio data in device memory.
	 */
	FAudioDescriptor Descriptor;

public:
	/**
	 * @return Get the descriptor of our audio data in device memory.
	 */
	inline FAudioDescriptor const& GetDescriptor() const { return Descriptor; }
	
	/**
	 * @return Get the number of samples in our data in device memory.
	 */
	inline float GetNumSamples() const { return Descriptor.NumSamples; }

	/**
	 * @return Get the duration of our data in device memory.
	 */
	inline float GetSampleDuration() const { return Descriptor.SampleDuration; }

	UE_API FAudioResource(TSharedRef<FAVDevice> const& Device, FAVLayout const& Layout, FAudioDescriptor const& Descriptor);
	virtual ~FAudioResource() override = default;
};

/**
 * Convenience wrapper for a audio device resource that requires a specific device context to function.
 */
template <typename TContext>
class TAudioResource : public FAudioResource
{
public:
	inline TSharedPtr<TContext> GetContext() const { return GetDevice()->template GetContext<TContext>(); }
	
	TAudioResource(TSharedRef<FAVDevice> const& Device, FAVLayout const& Layout, FAudioDescriptor const& Descriptor)
		: FAudioResource(Device, Layout, Descriptor)
	{
	}
};

/**
 * Wrapper for resolvable audio resources.
 */
template <typename TResource>
using TResolvableAudioResource = TResolvable<TResource, TSharedPtr<FAVDevice>, FAudioDescriptor>;

/**
 * Wrapper for delegated resolvable audio resources.
 */
template <typename TResource>
using TDelegatedAudioResource = TDelegated<TResource, TSharedPtr<FAVDevice>, FAudioDescriptor>;

/**
 * A simple pool-based resolvable audio resource. The contents of the pool must be manually managed by the application.
 */
template <typename TResource>
class TPooledAudioResource : public TResolvableAudioResource<TResource>
{
public:
	TArray<TSharedPtr<TResource>> Pool;

	virtual bool IsResolved(TSharedPtr<FAVDevice> const& Device, FAudioDescriptor const& Descriptor) const override
	{
		return TResolvableAudioResource<TResource>::IsResolved(Device, Descriptor) && Pool.Contains(this->Get());
	}

protected:
	virtual TSharedPtr<TResource> TryResolve(TSharedPtr<FAVDevice> const& Device, FAudioDescriptor const& Descriptor) override
	{
		for (int i = 0; i < Pool.Num(); ++i)
		{
			if (Pool[i]->GetDevice() == Device && Pool[i]->GetDescriptor() == Descriptor)
			{
				return Pool[i];
			}
		}

		return nullptr;
	}
};

#undef UE_API
