// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Audio/AudioEncoder.h"

/*
 * Configuration settings for AAC encoders.
 */
struct FAudioEncoderConfigAAC : public FAudioEncoderConfig
{
public:
	FAudioEncoderConfigAAC(EAVPreset Preset = EAVPreset::Default)
		: FAudioEncoderConfig(Preset)
	{
	}
};

DECLARE_TYPEID(FAudioEncoderConfigAAC, AVCODECSCORE_API);
