// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVContext.h"
#include "Audio/AudioResource.h"

#define UE_API AVCODECSCORE_API

/**
 * CPU audio context and resource.
 */

class FAudioContextCPU : public FAVContext
{
public:
	UE_API FAudioContextCPU();
};

class FAudioResourceCPU : public TAudioResource<FAudioContextCPU>
{
private:
	TSharedPtr<float> Raw;

public:
	inline TSharedPtr<float> const& GetRaw() const { return Raw; }

	UE_API FAudioResourceCPU(TSharedRef<FAVDevice> const& Device, TSharedPtr<float> const& Raw, FAVLayout const& Layout, FAudioDescriptor const& Descriptor);
	virtual ~FAudioResourceCPU() override = default;

	UE_API virtual FAVResult Validate() const override;
};

DECLARE_TYPEID(FAudioContextCPU, AVCODECSCORE_API);
DECLARE_TYPEID(FAudioResourceCPU, AVCODECSCORE_API);

#undef UE_API
