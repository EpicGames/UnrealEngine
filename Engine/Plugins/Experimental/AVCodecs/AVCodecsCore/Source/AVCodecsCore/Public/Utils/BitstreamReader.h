// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if defined(_MSC_VER)
#include <intrin.h>
#endif

// Borrowed from Plugins/Media/ElectraPlayer/Source/ElectraPlayerRuntime/Private/Runtime/BitDataStream.h

template <typename Type>
struct TBitSizeOf
{
	static constexpr uint16 Value = sizeof(Type) * 8;
};

template <uint8 NumBits>
using TBitSizeContainer = typename TTupleElement<(NumBits > 8) + (NumBits > 16) + (NumBits > 32), TTuple<uint8, uint16, uint32, uint64>>::Type;

/*
 * Base class for a compressed block of bitstream data. Provides scoped access to some parsing types for use with FBitstreamReader.
 */
struct FBitstreamSegment
{
	/*
	 * Unsigned int, compressed to a fixed number of bits.
	 */
	template <uint8 NumBits = 0, typename ValueType = TBitSizeContainer<NumBits>>
	struct U
	{
	public:
		ValueType Value = ValueType();

		U() = default;
		U(ValueType const& Value) : Value(Value) {}

		operator ValueType() const
		{
			return Value;
		}

		bool AsBool() const
		{
			return Value != 0;
		}

		U& operator=(ValueType const& From)
		{
			Value = From;
			
			return *this;
		}
	};

	/*
	 * Unsigned int, compressed to a dynamic number of bits.
	 */
	template <typename ValueType = uint32>
	struct UnsignedExpGolomb
	{
	public:
		ValueType Value = static_cast<ValueType>(0);

		UnsignedExpGolomb() = default;
		UnsignedExpGolomb(ValueType Value) : Value(Value) {}

		operator ValueType() const
		{
			return Value;
		}

		UnsignedExpGolomb& operator=(ValueType From)
		{
			Value = From;
		
			return *this;
		}
	};

	using UE = UnsignedExpGolomb<>;

	/*
	 * Signed int, compressed to a dynamic number of bits.
	 */
	template <typename ValueType = int32>
	struct SignedExpGolomb
	{
	public:
		ValueType Value = static_cast<ValueType>(0);

		SignedExpGolomb() = default;
		SignedExpGolomb(ValueType Value) : Value(Value) {}

		operator ValueType() const
		{
			return Value;
		}

		SignedExpGolomb& operator=(ValueType From)
		{
			Value = From;
			
			return *this;
		}
	};
	
	using SE = SignedExpGolomb<>;
};

/*
 * Simple wrapper for typed reads from a bitstream.
 */
struct FBitstreamReader
{
public:
	FBitstreamReader(uint8* Data, uint64 DataSize, uint64 BytePosition = 0, uint8 BitPosition = 0)
		: Data(Data)
		, DataSize(DataSize)
		, BytePosition(BytePosition)
		, BitPosition(BitPosition)
	{
	}

	bool IsByteAligned() const
	{
		return (BitPosition & 7) == 0;
	}

	uint8* GetDataRemaining() const
	{
		return Data + BytePosition;
	}

	uint64 NumBytesRemaining() const
	{
		return DataSize - BytePosition;
	}

	uint64 NumBitsRemaining() const
	{
		return NumBytesRemaining() * 8 + (BitPosition ? 8 - BitPosition : 0);
	}

	void Seek(uint64 NewBytePosition, uint32 NewBitPosition = 0)
	{
		BytePosition = FMath::Min(DataSize, NewBytePosition);
		BitPosition = FMath::Min(8u, NewBitPosition);
	}

	void SkipBytes(uint64 NumBytes)
	{
		BytePosition += NumBytes;
	}

	void SkipBits(uint64 NumBits)
	{
		SkipBytes(NumBits >> 3);
		ReadBits(NumBits & 7);
	}

	uint32 PeekBits(uint64 NumBits) const
	{
		verifyf(NumBits <= 32, TEXT("This function can return at most 32 bits!"));

		if (NumBits == 0)
		{
			return 0;
		}
		else
		{
			uint32 PeekBytePosition = BytePosition + (BitPosition >> 3);
			uint32 PeekBitPosition = BitPosition & 7;

			uint32 Result;
			uint64 const PeekBytesRemaining = NumBytesRemaining();
			if (PeekBytesRemaining >= 4)
			{
#if defined(_MSC_VER)
				Result = _byteswap_ulong(*(uint32*)(Data + PeekBytePosition));
#else
				Result = __builtin_bswap32(*(uint32*)(Data + PeekBytePosition));
#endif
			}
			else
			{
				if (PeekBytesRemaining == 3)
				{
					Result = ((uint32)Data[PeekBytePosition + 0] << 24) | ((uint32)Data[PeekBytePosition + 1] << 16) | ((uint32)Data[PeekBytePosition + 2] << 8);
				}
				else if (PeekBytesRemaining == 2)
				{
					Result = ((uint32)Data[PeekBytePosition + 0] << 24) | ((uint32)Data[PeekBytePosition + 1] << 16);
				}
				else if (PeekBytesRemaining == 1)
				{
					Result = ((uint32)Data[PeekBytePosition + 0] << 24);
				}
				else
				{
					Result = 0;
				}

			}

			if (PeekBitPosition == 0)
			{
				Result = Result >> (32 - NumBits);
				return Result;
			}
			else
			{
				uint32 w0 = Result;
				uint8 w1;
				if (PeekBytesRemaining > 4)
				{
					w1 = *(uint8*)(Data + PeekBytePosition + 4);
				}
				else
				{
					w1 = 0;
				}
				Result = (w0 << PeekBitPosition) | uint32(w1 >> (8 - PeekBitPosition));
				Result = Result >> (32 - NumBits);
				return Result;
			}
		}
	}

	uint32 ReadBits(uint64 NumBits)
	{
		uint32 const Result = PeekBits(NumBits);
		
		uint32 const NewBitPosition = BitPosition + NumBits;
		BytePosition += NewBitPosition >> 3;
		BitPosition = NewBitPosition & 7;
		
		return Result;
	}

	uint64 ReadBits64(uint64 NumBits)
	{
		if (NumBits <= 32)
		{
			return ReadBits(NumBits);
		}
		else
		{
			uint32 const UpperResult = ReadBits(NumBits - 32);
			uint32 const LowerResult = ReadBits(32);

			return (uint64(UpperResult) << 32ULL) | uint64(LowerResult);
		}
	}

	template <typename TOutput>
	void ReadBits(TOutput& Output, uint64 NumBits)
	{
		verifyf(NumBits <= static_cast<uint64>(sizeof(TOutput)) * 8, TEXT("Attempted to read more bits than we can hold!"));

		uint64 const Bits = NumBits > 32 ? ReadBits64(NumBits) : ReadBits(NumBits);

		FMemory::Memcpy(&Output, &Bits, (NumBits + 7) >> 3);
	}

	template <typename TOutput>
	void Read(TOutput& Output)
	{
		ReadBits(Output, TBitSizeOf<TOutput>::Value);
	}
	
	template <typename ValueType>
	void Read(FBitstreamSegment::UnsignedExpGolomb<ValueType>& Output)
	{
		int32 lz = -1;
		for (uint32 b = 0; b == 0; ++lz)
		{
			b = ReadBits(1);
		}

		if (lz > 0)
		{
			Output = static_cast<ValueType>(((1 << lz) | ReadBits(lz)) - 1);
		}
		else
		{
			Output = static_cast<ValueType>(0u);
		}
	}
	
	template <typename ValueType>
	void Read(FBitstreamSegment::SignedExpGolomb<ValueType>& Output)
	{
		FBitstreamSegment::UE c;
		Read(c);

		Output = c & 1 ? ValueType((c + 1) >> 1) : -ValueType((c + 1) >> 1);	
	}

	template <typename TOutput, typename... TOutputs>
	void Read(TOutput& Output, TOutputs&... Outputs)
	{
		Read(Output);
		Read(Outputs...);
	}

private:
	uint8* Data;
	uint64 DataSize;

	uint64 BytePosition;
	uint8 BitPosition;
};

// Requires template param for ValueType, so that all instantiations of U<0> route through this (and more importantly, the deleted specialization of FBitstreamReader::Read)
template <typename ValueType>
struct FBitstreamSegment::U<0, ValueType>
{
public:
	uint32 Value = 0;

	U() = default;
	U(uint32 const& Value) : Value(Value) {}

	operator uint32() const
	{
		return Value;
	}

	U& operator=(uint32 From)
	{
		Value = From;
		return *this;
	}
};

template <int32 NumBits, typename ValueType>
struct TBitSizeOf<FBitstreamSegment::U<NumBits, ValueType>>
{
	static constexpr uint16 Value = NumBits;
};

template <>
void FBitstreamReader::Read(FBitstreamSegment::U<0>& Output) = delete;

struct FNalu : public FBitstreamSegment
{
};
