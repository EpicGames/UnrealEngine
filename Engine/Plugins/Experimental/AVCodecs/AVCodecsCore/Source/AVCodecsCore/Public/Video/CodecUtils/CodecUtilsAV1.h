// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Utils/BitstreamReader.h"

#include "AVResult.h"
#include "Video/VideoPacket.h"

enum class EAV1Profile : uint8
{
    Auto,
    Main,
    // We currently don't support these other profiles
    // High,
    // Professional,
	MAX
};