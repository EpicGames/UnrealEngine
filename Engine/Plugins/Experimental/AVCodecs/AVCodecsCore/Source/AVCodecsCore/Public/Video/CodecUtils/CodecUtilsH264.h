// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Utils/BitstreamReader.h"

#include "AVResult.h"

#include "CodecUtilsH264.generated.h"

// MAC is degined a "1" on the Mac platform. This conflicts with EH264VideoFormat::MAC = 4. 
#pragma push_macro("MAC")
#undef MAC

struct FH264ProfileDefinition;
struct FVideoPacket;

// Easy to understand user facing profiles from the T-Rec-H.264-201304 spec
// there is another enum in CodecUtilsH264 that aligns the profiles with PIC and constraints
// however users are not expected to understand those ones.

UENUM()
enum class EH264Profile : uint8
{
	Auto,
	CALVLC444Intra,
	Baseline,
	ConstrainedBaseline,
	Main,
	ScalableBaseline,
	ScalableConstrainedBaseline,
	ScalableHigh,
	ScalableConstrainedHigh,
	ScalableHighIntra,
	Extended,
	High,
	ProgressiveHigh,
	ConstrainedHigh,
	High10,
	High10Intra,
	MultiviewHigh,
	High422,
	High422Intra,
	StereoHigh,
	MultiviewDepthHigh,
	High444,
	High444Intra,
	MAX
};

enum class EH264AdaptiveTransformMode : uint8
{
	Auto,
	Disable,
	Enable
};

enum class EH264EntropyCodingMode : uint8
{
	Auto,
	CABAC,
	CAVLC
};

// Packetization modes are defined in RFC 6184 section 6
// Due to the structure containing this being initialized with zeroes
// in some places, and mode 1 being default, mode 1 needs to have the value
// zero.
enum class EH264PacketizationMode : uint8
{
	NonInterleaved = 0, // Mode 1 - STAP-A, FU-A is allowed
	SingleNalUnit		// Mode 0 - only single NALU allowed
};

extern AVCODECSCORE_API FH264ProfileDefinition GH264ProfileDefinitions[static_cast<uint8>(EH264Profile::MAX)];

namespace UE::AVCodecCore::H264
{
	enum class ENaluType : uint8
	{
		Unspecified = 0,
		SliceOfNonIdrPicture = 1,
		SliceDataPartitionA = 2,
		SliceDataPartitionB = 3,
		SliceDataPartitionC = 4,
		SliceIdrPicture = 5,
		SupplementalEnhancementInformation = 6,
		SequenceParameterSet = 7,
		PictureParameterSet = 8,
		AccessUnitDelimiter = 9,
		EndOfSequence = 10,
		EndOfStream = 11,
		FillerData = 12,
		SequenceParameterSetExtension = 13,
		PrefixNalUnit = 14,
		SubsetSequenceParameterSet = 15,
		Reserved16 = 16,
		Reserved17 = 17,
		Reserved18 = 18,
		SliceOfAnAuxiliaryCoded = 19,
		SliceExtension = 20,
		SliceExtensionForDepthView = 21,
		Reserved22 = 22,
		Reserved23 = 23,
		Unspecified24 = 24,
		Unspecified25 = 25,
		Unspecified26 = 26,
		Unspecified27 = 27,
		Unspecified28 = 28,
		Unspecified29 = 29,
		Unspecified30 = 30,
		Unspecified31 = 31
	};

	enum class EH264ProfileIDC : uint8
	{
		Auto = 0,
		CALVLC444Intra = 44,
		Baseline = 66,
		// ConstrainedBaseline = 66,			with constraint flag 1 set
		Main = 77,
		ScalableBaseline = 83,
		// ScalableConstrainedBaseline = 83,	with constraint flag 5 set
		ScalableHigh = 86,
		// ScalableConstrainedHigh = 86,		with constraint flag 5 set
		// ScalableHighIntra = 86,				with constraint flag 3 set
		Extended = 88,
		High = 100,
		// ProgressiveHigh = 100,				with constraint flag 4 set
		// ConstrainedHigh = 100,				with constraint flag 4 & 5 set
		High10 = 110,
		// High10Intra = 110,					with constraint flag 3 set
		MultiviewHigh = 118,
		High422 = 122,
		// High422Intra = 122,					with constraint flag 3 set
		StereoHigh = 128,
		MultiresolutionFrameCompatibleHigh = 134,
		MultiviewDepthHigh = 138,
		EnhancedMultiviewDepthHigh = 139,
		High444 = 244,
		// High444Intra = 244,					with constraint flag 3 set
	};

	enum class EH264ConstraintFlag : uint8
	{
		None = 0,
		Set0 = 1u << 0,
		Set1 = 1u << 1,
		Set2 = 1u << 2,
		Set3 = 1u << 3,
		Set4 = 1u << 4,
		Set5 = 1u << 5
	};

	inline EH264ConstraintFlag operator|(EH264ConstraintFlag const& lhs, EH264ConstraintFlag const& rhs)
	{
		return (EH264ConstraintFlag)(static_cast<uint8>(lhs) | static_cast<uint8>(rhs));
	}

	inline EH264ConstraintFlag operator&(EH264ConstraintFlag const& lhs, EH264ConstraintFlag const& rhs)
	{
		return (EH264ConstraintFlag)(static_cast<uint8>(lhs) & static_cast<uint8>(rhs));
	}

	inline EH264ConstraintFlag operator|=(EH264ConstraintFlag const& lhs, EH264ConstraintFlag const& rhs)
	{
		return (EH264ConstraintFlag)(static_cast<uint8>(lhs) | static_cast<uint8>(rhs));
	}

	inline bool operator==(EH264ConstraintFlag lhs, EH264ConstraintFlag rhs)
	{
		return static_cast<uint8>(lhs) == (static_cast<uint8>(lhs) & static_cast<uint8>(rhs));
	}

	inline uint8 operator&(uint8 const& lhs, EH264ConstraintFlag const& rhs)
	{
		return lhs & static_cast<uint8>(rhs);
	}

	enum class EH264Level : uint8
	{
		Level_1b = 0,
		Level_1 = 10,
		Level_1_1 = 11,
		Level_1_2 = 12,
		Level_1_3 = 13,
		Level_2 = 20,
		Level_2_1 = 21,
		Level_2_2 = 22,
		Level_3 = 30,
		Level_3_1 = 31,
		Level_3_2 = 32,
		Level_4 = 40,
		Level_4_1 = 41,
		Level_4_2 = 42,
		Level_5 = 50,
		Level_5_1 = 51,
		Level_5_2 = 52,
		// Level_6 = 60,
		// Level_6_1 = 61,
		// Level_6_2 = 62
	};

	enum class EH264AspectRatioIDC : uint8
	{
		Unspecified = 0,
		Square = 1, // 1x1
		_12x11 = 2,
		_10x11 = 3,
		_16x11 = 4,
		_40x33 = 5,
		_24x11 = 6,
		_20x11 = 7,
		_32x11 = 8,
		_80x33 = 9,
		_18x11 = 10,
		_15x11 = 11,
		_64x33 = 12,
		_160x99 = 13,
		_4x3 = 14,
		_3x2 = 15,
		_2x1 = 16,
		Reserved = 17,
		Extended_SAR = 255
	};

	enum class EH264VideoFormat : uint8
	{
		Component = 0,
		PAL = 1,
		NTSC = 2,
		SECAM = 3,
		MAC = 4,
		Unspecified = 5,
		Reserved6 = 6,
		Reserved7 = 7
	};
} // namespace UE::AVCodecCore::H264

struct FH264ProfileDefinition
{
	EH264Profile							   Profile;
	UE::AVCodecCore::H264::EH264ProfileIDC	   PIDC;
	UE::AVCodecCore::H264::EH264ConstraintFlag ConstraintFlags;
	const TCHAR* Name;
};

namespace UE::AVCodecCore::H264
{
	inline int32 EBSPtoRBSP(uint8* OutBuf, const uint8* InBuf, int32 NumBytesIn)
	{
		uint8* OutBase = OutBuf;
		while (NumBytesIn-- > 0)
		{
			uint8 b = *InBuf++;
			*OutBuf++ = b;
			if (b == 0)
			{
				if (NumBytesIn > 1)
				{
					if (InBuf[0] == 0x00 && InBuf[1] == 0x03)
					{
						*OutBuf++ = 0x00;
						InBuf += 2;
						NumBytesIn -= 2;
					}
				}
			}
		}
		return OutBuf - OutBase;
	}

	struct FNaluH264
	{
		uint64		 Start, Size;
		uint8		 StartCodeSize;
		uint8		 RefIdc;
		ENaluType	 Type;
		const uint8* Data;
	};

	AVCODECSCORE_API FAVResult FindNALUs(FVideoPacket const& InPacket, TArray<FNaluH264>& FoundNalus);

	constexpr uint8 Default_4x4_Intra[16] = {
		6, 13, 13, 20,
		20, 20, 28, 28,
		28, 28, 32, 32,
		32, 37, 37, 42
	};

	constexpr uint8 Default_4x4_Inter[16] = {
		10, 14, 14, 20,
		20, 20, 24, 24,
		24, 24, 27, 27,
		27, 30, 30, 34
	};

	constexpr uint8 Default_8x8_Intra[64] = {
		6, 10, 10, 13, 11, 13, 16, 16,
		16, 16, 18, 18, 18, 18, 18, 23,
		23, 23, 23, 23, 23, 25, 25, 25,
		25, 25, 25, 25, 27, 27, 27, 27,
		27, 27, 27, 27, 29, 29, 29, 29,
		29, 29, 29, 31, 31, 31, 31, 31,
		31, 33, 33, 33, 33, 33, 36, 36,
		36, 36, 38, 38, 38, 40, 40, 42
	};

	constexpr uint8 Default_8x8_Inter[64] = {
		9, 13, 13, 15, 13, 15, 17, 17,
		17, 17, 19, 19, 19, 19, 19, 21,
		21, 21, 21, 21, 21, 22, 22, 22,
		22, 22, 22, 22, 24, 24, 24, 24,
		24, 24, 24, 24, 25, 25, 25, 25,
		25, 25, 25, 27, 27, 27, 27, 27,
		27, 28, 28, 28, 28, 28, 30, 30,
		30, 30, 32, 32, 32, 33, 33, 35
	};

	constexpr uint8 ZigZag4x4[16] = {
		0, 1, 5, 6,
		2, 4, 7, 12,
		3, 8, 11, 13,
		9, 10, 15, 15
	};

	constexpr uint8 FieldScan4x4[16] = {
		0, 2, 8, 12,
		1, 5, 9, 13,
		3, 6, 10, 14,
		4, 7, 11, 15
	};

	constexpr uint8 ZigZag8x8[64] = {
		0, 1, 5, 6, 14, 15, 27, 28,
		2, 4, 7, 13, 16, 26, 29, 42,
		3, 8, 12, 17, 25, 30, 41, 43,
		9, 11, 18, 24, 31, 40, 44, 53,
		10, 19, 23, 32, 39, 45, 52, 54,
		20, 22, 33, 38, 46, 51, 55, 60,
		21, 34, 37, 47, 50, 56, 59, 61,
		35, 36, 48, 49, 57, 58, 62, 63
	};

	constexpr uint8 FieldScan8x8[64] = {
		0, 3, 8, 15, 22, 30, 38, 52,
		1, 4, 14, 21, 29, 37, 45, 53,
		2, 7, 16, 23, 31, 39, 46, 58,
		5, 9, 20, 28, 36, 44, 51, 59,
		6, 13, 24, 32, 40, 47, 54, 60,
		10, 17, 25, 33, 41, 48, 55, 61,
		11, 18, 26, 34, 42, 49, 56, 62,
		12, 19, 27, 35, 43, 50, 57, 63
	};

	inline void ScaleListToWeightScale(const bool bIsFrame, const uint8* ScalingList, const uint8 ListSize, uint8* OutWeightList /* called C in spec*/)
	{
		// OPTIMIZE (aidan) seems like this can be done faster but should be functional for now
		for (uint8 i = 0; i < ListSize; i++)
		{
			if (ListSize == 16)
			{
				OutWeightList[i] = ScalingList[bIsFrame ? ZigZag4x4[i] : FieldScan4x4[i]];
			}
			else
			{
				OutWeightList[i] = ScalingList[bIsFrame ? ZigZag8x8[i] : FieldScan8x8[i]];
			}
		}
	};

	struct SPS_t : public FNalu
	{
	public:
		U<8, EH264ProfileIDC>	  profile_idc = EH264ProfileIDC::Auto;
		U<8, EH264ConstraintFlag> constraint_flags = EH264ConstraintFlag::None;
		U<8>					  level_idc;							 // + profile_idc + constraint_flags = EH264Level
		UE						  seq_parameter_set_id;					 // 0...31 inclusive
		UE						  chroma_format_idc = 1;				 // 0...3 inclusive
		U<1>					  separate_colour_plane_flag;			 // bool
		UE						  bit_depth_luma_minus8;				 // 0...6 inclusive
		UE						  bit_depth_chroma_minus8;				 // 0...6 inclusive
		U<1>					  qpprime_y_zero_transform_bypass_flag;	 // 0...1 inclusive
		U<1>					  seq_scaling_matrix_present_flag;		 // bool
		U<1>					  seq_scaling_list_present_flag[12];	 // bool array
		uint8					  ScalingList4x4[6][16];				 // -2^(1+BitDepth)...2^(1+BitDepth)-1 inclusive
		uint8					  ScalingList8x8[6][64];				 // -2^(1+BitDepth)...2^(1+BitDepth)-1 inclusive
		UE						  log2_max_frame_num_minus4;			 // 0...12 inclusive
		UE						  pic_order_cnt_type;					 // 0...3 inclusive
		UE						  log2_max_pic_order_cnt_lsb_minus4;	 // 0...12 inclusive
		U<1>					  delta_pic_order_always_zero_flag;		 // bool
		SE						  offset_for_non_ref_pic;				 // −2^(31)+1...2^(31)−1 inclusive
		SE						  offset_for_top_to_bottom_field;		 // −2^(31)+1...2^(31)−1 inclusive
		UE						  num_ref_frames_in_pic_order_cnt_cycle; // 0...255 inclusive
		SE						  offset_for_ref_frame[255];			 // −2^(31)+1...2^(31)−1 inclusive
		UE						  max_num_ref_frames;					 // 0...MaxDpbFrames inclusive MaxDpbFrames = Min( MaxDecPicBufSizeMacro / ( PicWidthMacro * FrameHeightMacro ), 16) (A.3.1 or A.3.2)
		U<1>					  gaps_in_frame_num_value_allowed_flag;	 // bool
		UE						  pic_width_in_mbs_minus1;				 // + 1 = Width in macroblocks
		UE						  pic_height_in_map_units_minus1;		 // + 1 = Height
		U<1>					  frame_mbs_only_flag;					 // bool
		U<1>					  mb_adaptive_frame_field_flag;			 // bool
		U<1>					  direct_8x8_inference_flag;			 // bool
		U<1>					  frame_cropping_flag;					 // bool
		UE						  frame_crop_left_offset;
		UE						  frame_crop_right_offset;
		UE						  frame_crop_top_offset;
		UE						  frame_crop_bottom_offset;
		U<1>					  vui_parameters_present_flag; // bool
		// VUI Parameters
		U<1>					  aspect_ratio_info_present_flag; // bool
		U<8, EH264AspectRatioIDC> aspect_ratio_idc = EH264AspectRatioIDC::Unspecified;
		U<16>					  sar_width;					  // Width in arbitrary unit
		U<16>					  sar_height;					  // Height in same arbitrary unit as Width
		U<1>					  overscan_info_present_flag;	  // bool
		U<1>					  overscan_appropriate_flag;	  // bool
		U<1>					  video_signal_type_present_flag; // bool
		U<3, EH264VideoFormat>	  video_format = EH264VideoFormat::Unspecified;
		U<1>					  video_full_range_flag;				   // bool
		U<1>					  colour_description_present_flag;		   // bool
		U<8>					  colour_primaries = 2;					   // 0...255 inclusive but see Table E-3 as many values are Reserved
		U<8>					  transfer_characteristics = 2;			   // 0...255 inclusive but see Table E-4 as many values are Reserved
		U<8>					  matrix_coefficients = 2;				   // 0...255 inclusive but see Table E-5 as many values are Reserved
		U<1>					  chroma_loc_info_present_flag;			   // bool
		UE						  chroma_sample_loc_type_top_field;		   // 0...5 inclusive
		UE						  chroma_sample_loc_type_bottom_field;	   // 0...5 inclusive
		U<1>					  timing_info_present_flag;				   // bool
		U<32>					  num_units_in_tick = 1;				   // > 0
		U<32>					  time_scale = 1000;					   // defaulted to ms
		U<1>					  fixed_frame_rate_flag;				   // bool
		U<1>					  nal_hrd_parameters_present_flag;		   // bool
		U<1>					  vcl_hrd_parameters_present_flag;		   // bool
		U<1>					  low_delay_hrd_flag;					   // bool
		U<1>					  pic_struct_present_flag;				   // bool
		U<1>					  bitstream_restriction_flag;			   // bool
		U<1>					  motion_vectors_over_pic_boundaries_flag; // bool
		UE						  max_bytes_per_pic_denom;
		UE						  max_bits_per_mb_denom;
		UE						  log2_max_mv_length_horizontal;
		UE						  log2_max_mv_length_vertical;
		UE						  max_num_reorder_frames;
		UE						  max_dec_frame_buffering;
		// HRD Parameters
		UE	 cpb_cnt_minus1; // 0...31 inclusive
		U<4> bit_rate_scale;
		U<4> cpb_size_scale;
		UE	 bit_rate_value_minus1[31];					   // 0...2^(32)−2 inclusive TODO (aidan) might need an infer stage to fill out based on profile
		UE	 cpb_size_value_minus1[31];					   // 0...2^(32)−2 inclusive TODO (aidan) might need an infer stage to fill out based on profile
		U<1> cbr_flag[31];								   // bool
		U<5> initial_cpb_removal_delay_length_minus1 = 23; //
		U<5> cpb_removal_delay_length_minus1 = 23;
		U<5> dpb_output_delay_length_minus1 = 23;
		U<5> time_offset_length = 24;

		EH264Profile GetProfile() const
		{
			for (FH264ProfileDefinition const& H264Profile : GH264ProfileDefinitions)
			{
				if (profile_idc == H264Profile.PIDC && (constraint_flags == H264Profile.ConstraintFlags))
				{
					return H264Profile.Profile;
				}
			}

			return EH264Profile::MAX;
		}
	};

	FAVResult ParseSPS(FBitstreamReader& Bitstream, FNaluH264 const& InNaluInfo, TMap<uint32, SPS_t>& OutMapSPS);

	struct PPS_t : public FNalu
	{
		UE			pic_parameter_set_id;
		UE			seq_parameter_set_id;
		U<1>		entropy_coding_mode_flag;
		U<1>		bottom_field_pic_order_in_frame_present_flag;
		UE			num_slice_groups_minus1;
		UE			slice_group_map_type;
		TArray<UE>	run_length_minus1; // hard to justify preallocating this as it could be PicWidthInMbs * PicHeightInMapUnits - 1
		TArray<UE>	top_left;		   // hard to justify preallocating this as it could be PicWidthInMbs * PicHeightInMapUnits - 1
		TArray<UE>	bottom_right;	   // hard to justify preallocating this as it could be PicWidthInMbs * PicHeightInMapUnits - 1
		U<1>		slice_group_change_direction_flag;
		UE			slice_group_change_rate_minus1;
		UE			pic_size_in_map_units_minus1;
		TArray<U<>> slice_group_id; // hard to justify preallocating this as it could be pic_size_in_map_units_minus1 + 1
		UE			num_ref_idx_l0_default_active_minus1;
		UE			num_ref_idx_l1_default_active_minus1;
		U<1>		weighted_pred_flag;
		U<2>		weighted_bipred_idc;
		SE			pic_init_qp_minus26;
		SE			pic_init_qs_minus26;
		SE			chroma_qp_index_offset;
		U<1>		deblocking_filter_control_present_flag;
		U<1>		constrained_intra_pred_flag;
		U<1>		redundant_pic_cnt_present_flag;
		U<1>		transform_8x8_mode_flag;
		U<1>		pic_scaling_matrix_present_flag;
		U<1>		pic_scaling_list_present_flag[12];
		uint8		ScalingList4x4[6][16]; // -2^(1+BitDepth)...2^(1+BitDepth)-1 inclusive
		uint8		ScalingList8x8[6][64]; // -2^(1+BitDepth)...2^(1+BitDepth)-1 inclusive
		SE			second_chroma_qp_index_offset;
	};

	FAVResult ParsePPS(FBitstreamReader& Bitstream, FNaluH264 const& InNaluInfo, TMap<uint32, SPS_t> const& InMapSPS, TMap<uint32, PPS_t>& OutMapPPS);

	struct Slice_t : public FNalu
	{
		UE	 first_mb_in_slice;
		UE	 slice_type;
		UE	 pic_parameter_set_id;
		U<2> colour_plane_id;
		U<>	 frame_num;
		U<1> field_pic_flag;
		U<1> bottom_field_flag;
		UE	 idr_pic_id;
		U<>	 pic_order_cnt_lsb;
		SE	 delta_pic_order_cnt_bottom;
		SE	 delta_pic_order_cnt[2];
		UE	 redundant_pic_cnt;
		U<1> direct_spatial_mv_pred_flag;
		U<1> num_ref_idx_active_override_flag;
		UE	 num_ref_idx_l0_active_minus1;
		UE	 num_ref_idx_l1_active_minus1;
		UE	 cabac_init_idc;
		SE	 slice_qp_delta;
		U<1> sp_for_switch_flag;
		SE	 slice_qs_delta;
		UE	 disable_deblocking_filter_idc;
		SE	 slice_alpha_c0_offset_div2;
		SE	 slice_beta_offset_div2;
		U<>	 slice_group_change_cycle;

		// ref_pic_list_modification
		struct RefPic_t
		{
			U<1> bIsLongTerm;
			UE	 pic_num;
		};

		U<1> ref_pic_list_modification_flag_l0;
		// TArray<RefPic_t> RefPicList0;
		U<1> ref_pic_list_modification_flag_l1;
		// TArray<RefPic_t> RefPicList1;

		// pred_weight_table

		// dec_ref_pic_marking
		U<1> no_output_of_prior_pic_flag;
		U<1> long_term_reference_flag;

		U<1> adaptive_ref_pic_marking_mode_flag;

		UE difference_of_pic_nums_minus1;
		UE long_term_pic_num;
		UE long_term_frame_idx;
		UE max_long_term_frame_idx_plus1;
	};

	FAVResult ParseSliceHeader(FBitstreamReader& Bitstream, FNaluH264 const& InNaluInfo, TMap<uint32, SPS_t> const& InMapSPS, TMap<uint32, PPS_t> const& InMapPPS, Slice_t& OutSlice);

	// SEI can transmit arbitrary data so we probably want something less rigid for this
	struct SEI_t
	{
	};

	FAVResult ParseSEI(FBitstreamReader& Bitstream, FNaluH264 const& InNaluInfo, SEI_t& OutSEI);
} // namespace UE::AVCodecCore::H264

#pragma pop_macro("MAC")