// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Utils/BitstreamReader.h"

#include "Containers/Array.h"
#include "Containers/StaticArray.h"
#include "AVResult.h"
#include "Math/MathFwd.h"

struct FH265ProfileDefinition;
struct FVideoPacket;
struct FVideoDecoderConfigH265;

// Easy to understand user facing profiles from the T-Rec-H.264-201304 spec
// there is another enum in CodecUtilsH265 that aligns the profiles with PIC and constraints
// however users are not expected to understand those ones.
enum class EH265Profile : uint8
{
	Auto,
	Main,
	Main10,
	Main10StillPicture,
	MainStillPicture,
	Monochrome,
	Monochrome10,
	Monochrome12,
	Monochrome16,
	Main12,
	Main422_10,
	Main422_12,
	Main444,
	Main444_10,
	Main444_12,
	MainIntra,
	Main10Intra,
	Main12Intra,
	Main422_10Intra,
	Main422_12Intra,
	Main444_Intra,
	Main444_10Intra,
	Main444_12Intra,
	Main444_16Intra,
	Main444StillPicture,
	Main444_16StillPicture,
	HighThroughput444,
	HighThroughput444_10,
	HighThroughput444_14,
	HighThroughput444_16Intra,
	ScreenExtendedMain,
	ScreenExtendedMain10,
	ScreenExtendedMain444,
	ScreenExtendedMain444_10,
	ScreenExtendedHighThroughput444,
	ScreenExtendedHighThroughput444_10,
	ScreenExtendedHighThroughput444_14,
	MAX
};

extern AVCODECSCORE_API FH265ProfileDefinition GH265ProfileDefinitions[static_cast<uint8>(EH265Profile::MAX)];

namespace UE::AVCodecCore::H265
{
	enum class ENaluType : uint8
	{
		TRAIL_N = 0,		 // Coded slice segment of a non-TSA, non-STSA trailing picture
		TRAIL_R = 1,		 // Coded slice segment of a non-TSA, non-STSA trailing picture
		TSA_N = 2,			 // Coded slice segment of a TSA picture
		TSA_R = 3,			 // Coded slice segment of a TSA picture
		STSA_N = 4,			 // Coded slice segment of an STSA picture
		STSA_R = 5,			 // Coded slice segment of an STSA picture
		RADL_N = 6,			 // Coded slice segment of a RADL picture
		RADL_R = 7,			 // Coded slice segment of a RADL picture
		RASL_N = 8,			 // Coded slice segment of a RASL picture
		RASL_R = 9,			 // Coded slice segment of a RASL picture
		RSV_VCL_N10 = 10,	 // Reserved non-IRAP SLNR VCL NAL unit types
		RSV_VCL_R11 = 11,	 // Reserved non-IRAP sub-layer reference VCL NAL unit types
		RSV_VCL_N12 = 12,	 // Reserved non-IRAP SLNR VCL NAL unit types
		RSV_VCL_R13 = 13,	 // Reserved non-IRAP sub-layer reference VCL NAL unit types
		RSV_VCL_N14 = 14,	 // Reserved non-IRAP SLNR VCL NAL unit types
		RSV_VCL_R15 = 15,	 // Reserved non-IRAP sub-layer reference VCL NAL unit types
		BLA_W_LP = 16,		 // Coded slice segment of a BLA picture
		BLA_W_RADL = 17,	 // Coded slice segment of a BLA picture
		BLA_N_LP = 18,		 // Coded slice segment of a BLA picture
		IDR_W_RADL = 19,	 // Coded slice segment of a IDR picture
		IDR_N_LP = 20,		 // Coded slice segment of a IDR picture
		CRA_NUT = 21,		 // Coded slice segment of a CRA picture
		RSV_IRAP_VCL22 = 22, // Reserved IRAP VCL NAL unit types
		RSV_IRAP_VCL23 = 23, // Reserved IRAP VCL NAL unit types
		RSV_VCL24 = 24,		 // Reserved non-IRAP VCL NAL unit types
		RSV_VCL25 = 25,		 // Reserved non-IRAP VCL NAL unit types
		RSV_VCL26 = 26,		 // Reserved non-IRAP VCL NAL unit types
		RSV_VCL27 = 27,		 // Reserved non-IRAP VCL NAL unit types
		RSV_VCL28 = 28,		 // Reserved non-IRAP VCL NAL unit types
		RSV_VCL29 = 29,		 // Reserved non-IRAP VCL NAL unit types
		RSV_VCL30 = 30,		 // Reserved non-IRAP VCL NAL unit types
		RSV_VCL31 = 31,		 // Reserved non-IRAP VCL NAL unit types
		VPS_NUT = 32,		 // Video Parameter Set
		SPS_NUT = 33,		 // Sequence Parameter Set
		PPS_NUT = 34,		 // Picture Parameter Set
		AUD_NUT = 35,		 // Access Unit Delimiter
		EOS_NUT = 36,		 // End of Sequence
		EOB_NUT = 37,		 // End of Bitstream
		FD_NUT = 38,		 // Filler data
		PREFIX_SEI_NUT = 39, // Supplemental Enhancement information
		SUFFIX_SEI_NUT = 40, // Supplemental Enhancement information
		RSV_NVCL41 = 41,	 // Reserved
		RSV_NVCL42 = 42,	 // Reserved
		RSV_NVCL43 = 43,	 // Reserved
		RSV_NVCL44 = 44,	 // Reserved
		RSV_NVCL45 = 45,	 // Reserved
		RSV_NVCL46 = 46,	 // Reserved
		RSV_NVCL47 = 47,	 // Reserved
		UNSPECIFIED = 48,	 // Unspecified (anything > than this)
	};

	enum class EH265ProfileIDC : uint8
	{
		Auto = 0,
		Main = 1,
		Main10 = 2,
		Main10StillPicture = Main10,
		MainStillPicture = 3,
		FormatRangeExtensions = 4,
		Monochrome = FormatRangeExtensions,
		Monochrome10 = FormatRangeExtensions,
		Monochrome12 = FormatRangeExtensions,
		Monochrome16 = FormatRangeExtensions,
		Main12 = FormatRangeExtensions,
		Main422_10 = FormatRangeExtensions,
		Main422_12 = FormatRangeExtensions,
		Main444 = FormatRangeExtensions,
		Main444_10 = FormatRangeExtensions,
		Main444_12 = FormatRangeExtensions,
		MainIntra = FormatRangeExtensions,
		Main10Intra = FormatRangeExtensions,
		Main12Intra = FormatRangeExtensions,
		Main422_10Intra = FormatRangeExtensions,
		Main422_12Intra = FormatRangeExtensions,
		Main444_Intra = FormatRangeExtensions,
		Main444_10Intra = FormatRangeExtensions,
		Main444_12Intra = FormatRangeExtensions,
		Main444_16Intra = FormatRangeExtensions,
		Main444StillPicture = FormatRangeExtensions,
		Main444_16StillPicture = FormatRangeExtensions,
		HighThroughput = 5,
		HighThroughput444 = HighThroughput,
		HighThroughput444_10 = HighThroughput,
		HighThroughput444_14 = HighThroughput,
		HighThroughput444_16Intra = HighThroughput,
        MultiViewMain = 6,
        ScalableMain = 7,
        ThreeDimensionalMain = 8,
		ScreenContentCoding = 9,
		ScreenExtendedMain = ScreenContentCoding,
		ScreenExtendedMain10 = ScreenContentCoding,
		ScreenExtendedMain444 = ScreenContentCoding,
		ScreenExtendedMain444_10 = ScreenContentCoding,
        ScalableRangeExtensions = 10,
		HighThroughputScreenContentCoding = 11,
		ScreenExtendedHighThroughput444 = HighThroughputScreenContentCoding,
		ScreenExtendedHighThroughput444_10 = HighThroughputScreenContentCoding,
		ScreenExtendedHighThroughput444_14 = HighThroughputScreenContentCoding,
	};

	inline bool CheckProfileCompatabilityFlag(uint32 const& profile_compatibility_flag, EH265ProfileIDC const& H265ProfileIdc)
	{
		const uint32 tempFlag = 1U << static_cast<uint32>(H265ProfileIdc);
		return (profile_compatibility_flag & tempFlag) == tempFlag;
	}

	enum class EH265ConstraintFlag : uint16
	{
		None = 0,
		non_packed_constraint_flag = 1u << 0,
		frame_only_constraint_flag = 1u << 1,
		max_12bit_constraint_flag = 1u << 2,
		max_10bit_constraint_flag = 1u << 3,
		max_8bit_constraint_flag = 1u << 4,
		max_422chroma_constraint_flag = 1u << 5,
		max_420chroma_constraint_flag = 1u << 6,
		max_monochrome_constraint_flag = 1u << 7,
		intra_constraint_flag = 1u << 8,
		one_picture_only_constraint_flag = 1u << 9,
		lower_bit_rate_constraint_flag = 1u << 10,
		max_14bit_constraint_flag = 1u << 11
	};

	inline EH265ConstraintFlag operator|(EH265ConstraintFlag const& lhs, EH265ConstraintFlag const& rhs)
	{
		return (EH265ConstraintFlag)(static_cast<uint16>(lhs) | static_cast<uint16>(rhs));
	}

	inline EH265ConstraintFlag operator&(EH265ConstraintFlag const& lhs, EH265ConstraintFlag const& rhs)
	{
		return (EH265ConstraintFlag)(static_cast<uint16>(lhs) & static_cast<uint16>(rhs));
	}

	inline EH265ConstraintFlag operator|=(EH265ConstraintFlag const& lhs, EH265ConstraintFlag const& rhs)
	{
		return (EH265ConstraintFlag)(static_cast<uint16>(lhs) | static_cast<uint16>(rhs));
	}

	inline bool operator==(EH265ConstraintFlag lhs, EH265ConstraintFlag rhs)
	{
		return static_cast<uint16>(lhs) == (static_cast<uint16>(lhs) & static_cast<uint8>(rhs));
	}

	inline uint8 operator&(uint16 const& lhs, EH265ConstraintFlag const& rhs)
	{
		return lhs & static_cast<uint16>(rhs);
	}

	enum class EH265LevelIDC : uint8
	{
		Level_1 = 10,
		Level_2 = 20,
		Level_2_1 = 21,
		Level_3 = 30,
		Level_3_1 = 31,
		Level_4 = 40,
		Level_4_1 = 41,
		Level_5 = 50,
		Level_5_1 = 51,
		Level_5_2 = 52,
		Level_6 = 60,
		Level_6_1 = 61,
		Level_6_2 = 62
	};

	enum class EH265AspectRatioIDC : uint8
	{
		Unspecified = 0,
		Square = 1, // 1x1
		_12x11 = 2,
		_10x11 = 3,
		_16x11 = 4,
		_40x33 = 5,
		_24x11 = 6,
		_20x11 = 7,
		_32x11 = 8,
		_80x33 = 9,
		_18x11 = 10,
		_15x11 = 11,
		_64x33 = 12,
		_160x99 = 13,
		_4x3 = 14,
		_3x2 = 15,
		_2x1 = 16,
		Reserved = 17,
		Extended_SAR = 255
	};

	enum class EH265VideoFormat : uint8
	{
		Component = 0,
		PAL = 1,
		NTSC = 2,
		SECAM = 3,
		MAC = 4,
		Unspecified = 5,
		Reserved6 = 6,
		Reserved7 = 7
	};
} // namespace UE::AVCodecCore::H265

struct FH265ProfileDefinition
{
	EH265Profile Profile;
	UE::AVCodecCore::H265::EH265ProfileIDC PIDC;
	UE::AVCodecCore::H265::EH265ConstraintFlag ConstraintFlags;
	const TCHAR* Name;
};

namespace UE::AVCodecCore::H265
{
	inline int32 EBSPtoRBSP(uint8* OutBuf, const uint8* InBuf, int32 NumBytesIn)
	{
		uint8* OutBase = OutBuf;
		while (NumBytesIn-- > 0)
		{
			uint8 b = *InBuf++;
			*OutBuf++ = b;
			if (b == 0)
			{
				if (NumBytesIn > 1)
				{
					if (InBuf[0] == 0x00 && InBuf[1] == 0x03)
					{
						*OutBuf++ = 0x00;
						InBuf += 2;
						NumBytesIn -= 2;
					}
				}
			}
		}
		return OutBuf - OutBase;
	}

	struct FNaluH265
	{
		uint64 Start, Size;
		uint8 StartCodeSize;
        ENaluType Type;
		uint8 NuhLayerId;
        uint8 NuhTemporalIdPlus1;
        const uint8* Data;
	};

	AVCODECSCORE_API FAVResult FindNALUs(FVideoPacket const& InPacket, TArray<FNaluH265>& FoundNalus);

	struct profile_tier_level_t : public FBitstreamSegment
	{
		U<2> general_profile_space;
		U<1> general_tier_flag;
		U<5, EH265ProfileIDC> general_profile_idc;
        TArray<U<1>> general_profile_compatibility_flag;
		U<1> general_progressive_source_flag;
		U<1> general_interlaced_source_flag;
        U<1> general_non_packed_constraint_flag;
        U<1> general_frame_only_constraint_flag;


        U<1> general_max_12bit_constraint_flag;
        U<1> general_max_10bit_constraint_flag;
        U<1> general_max_8bit_constraint_flag;
        U<1> general_max_422chroma_constraint_flag;
        U<1> general_max_420chroma_constraint_flag;
        U<1> general_max_monochrome_constraint_flag;
        U<1> general_intra_constraint_flag;
        U<1> general_one_picture_only_constraint_flag;
        U<1> general_lower_bit_rate_constraint_flag;
        U<1> general_max_14bit_constraint_flag;

		U<1> general_inbld_flag = 0;
		U<8, EH265LevelIDC> general_level_idc;

		struct sub_layer_t
		{
			U<1> sub_layer_profile_present_flag;
			U<1> sub_layer_level_present_flag;

			U<2> sub_layer_profile_space;
			U<1> sub_layer_tier_flag = 0;
			U<5, EH265ProfileIDC> sub_layer_profile_idc;
			U<1> sub_layer_progressive_source_flag;
			U<1> sub_layer_interlaced_source_flag;
            TArray<U<1>> sub_layer_profile_compatibility_flag;
            U<1> sub_layer_non_packed_constraint_flag;
            U<1> sub_layer_frame_only_constraint_flag;

            U<1> sub_layer_max_12bit_constraint_flag;
            U<1> sub_layer_max_10bit_constraint_flag;
            U<1> sub_layer_max_8bit_constraint_flag;
            U<1> sub_layer_max_422chroma_constraint_flag;
            U<1> sub_layer_max_420chroma_constraint_flag;
            U<1> sub_layer_max_monochrome_constraint_flag;
            U<1> sub_layer_intra_constraint_flag;
            U<1> sub_layer_one_picture_only_constraint_flag;
            U<1> sub_layer_lower_bit_rate_constraint_flag;
            U<1> sub_layer_max_14bit_constraint_flag;

			U<1> sub_layer_inbld_flag;
			U<8, EH265LevelIDC> sub_layer_level_idc;
		};

		TArray<sub_layer_t> sub_layers;

        inline void Parse(uint8 const& profilePresentFlag, uint8 maxNumSubLayersMinus1, FBitstreamReader& Bitstream);
	};

	struct hrd_parameters_t : public FBitstreamSegment
	{
		U<1> nal_hrd_parameters_present_flag;
		U<1> vcl_hrd_parameters_present_flag;
		U<1> sub_pic_hrd_params_present_flag;
		U<8> tick_divisor_minus2;
		U<5> du_cpb_removal_delay_increment_length_minus1;
		U<1> sub_pic_cpb_params_in_pic_timing_sei_flag;
		U<5> dpb_output_delay_du_length_minus1;
		U<4> bit_rate_scale;
		U<4> cpb_size_scale;
		U<4> cpb_size_du_scale;
		U<5> initial_cpb_removal_delay_length_minus1;
		U<5> au_cpb_removal_delay_length_minus1;
		U<5> dpb_output_delay_length_minus1;

		struct sub_layer_t
		{
			U<1> fixed_pic_rate_general_flag;
			U<1> fixed_pic_rate_within_cvs_flag;
			UE elemental_duration_in_tc_minus1;
			U<1> low_delay_hrd_flag;
			UE cpb_cnt_minus1;

			struct sub_layer_hrd_parameters_t
			{
				UE bit_rate_value_minus1;
				UE cpb_size_value_minus1;
				UE cpb_size_du_value_minus1;
				UE bit_rate_du_value_minus1;
				U<1> cbr_flag;
			};

			TArray<sub_layer_hrd_parameters_t> sub_layer_hrd_parameters;

            inline void Parse(uint8 const& in_sub_pic_hrd_params_present_flag, uint8 const& CpbCnt, FBitstreamReader& Bitstream);
        };

		TArray<sub_layer_t> sub_layers;

        inline void Parse(uint8 const& commonInfPresentFlag, uint8 const& maxNumSubLayersMinus1, FBitstreamReader& Bitstream);
    };

	struct VPS_t : public FNalu
	{
		U<4> vps_video_parameter_set_id;
		U<1> vps_base_layer_internal_flag;
		U<1> vps_base_layer_available_flag;
		U<6> vps_max_layers_minus1;
		U<3> vps_max_sub_layers_minus1;
		U<1> vps_temporal_id_nesting_flag;

		profile_tier_level_t profile_tier_level;

		U<1> vps_sub_layer_ordering_info_present_flag;
		TArray<UE> vps_max_dec_pic_buffering_minus1;
		TArray<UE> vps_max_num_reorder_pics;
		TArray<UE> vps_max_latency_increase_plus1;
		U<6> vps_max_layer_id;
		UE vps_num_layer_sets_minus1;
		TArray<TArray<U<1>>> layer_id_included_flag;
		U<1> vps_timing_info_present_flag;
		U<32> vps_num_units_in_tick;
		U<32> vps_time_scale;
		U<1> vps_poc_proportional_to_timing_flag;
		UE vps_num_ticks_poc_diff_one_minus1;
		UE vps_num_hrd_parameters;
		TArray<UE> hrd_layer_set_idx;
		TArray<U<1>> cprms_present_flag;
		TArray<hrd_parameters_t> hrd_parameters;
		U<1> vps_extension_flag;

		// Stream State Members
		// TODO (aidan) perhaps this should be moved elsewhere
		mutable uint32 prevPicOrderCntLsb = 0;
		mutable uint32 prevPicOrderCntMsb = 0;
	};

    FAVResult ParseVPS(FBitstreamReader& Bitstream, FNaluH265 const& InNaluInfo, TMap<uint32, VPS_t>& OutMapVPS);

	
	template <uint32 Tlog2blkSize, uint32 TscanIdx>
	struct TScanOrder
	{
private:
		static constexpr uint32 TBlockSize = 1u << Tlog2blkSize;
		
		// TODO: This algorithm doesn't make sense, fix
		static constexpr TStaticArray<FIntPoint, TBlockSize * TBlockSize> MakeUpperRightDiagonal()
		{
			TStaticArray<FIntPoint, TBlockSize * TBlockSize> diagScan;			
			// int32 i = 0, x = 0, y = 0;
			// do
			// {
			// 	while (y >= 0)
			// 	{
			// 		if (x < TBlockSize && y < TBlockSize)
			// 		{
			// 			diagScan[i] = { x, y };
			// 			i++;
			// 		}
			// 	}
			// 	y = x;
			// 	x = 0;
			// 	if (i >= TBlockSize * TBlockSize)
			// 	{
			// 		break;
			// 	}
			// } while (i >= TBlockSize * TBlockSize);
			
			return diagScan;
		}

		static constexpr TStaticArray<FIntPoint, TBlockSize * TBlockSize> MakeHorizontal()
		{
			TStaticArray<FIntPoint, TBlockSize * TBlockSize> horScan;			
			uint32 i = 0;
			for(int32 y = 0; y < TBlockSize; y++)
			{
				for(int32 x = 0; x < TBlockSize; x++)
				{
					horScan[i] = { x, y };
				}
			}
			return horScan;			
		}
				
		static constexpr TStaticArray<FIntPoint, TBlockSize * TBlockSize> MakeVertical()
		{
			TStaticArray<FIntPoint, TBlockSize * TBlockSize> verScan;			
			uint32 i = 0;
			for(int32 x = 0; x < TBlockSize; x++)
			{
				for(int32 y = 0; y < TBlockSize; y++)
				{
					verScan[i] = { x, y };
				}
			}
			return verScan;			
		}
		
		static constexpr TStaticArray<FIntPoint, TBlockSize * TBlockSize> MakeTraverse()
		{
			TStaticArray<FIntPoint, TBlockSize * TBlockSize> travScan;			
			uint32 i = 0;			
			for (int32 y = 0; y < TBlockSize; y++)
			{
				if (y % 2 == 0)
				{
					for(int32 x = 0; x < TBlockSize; x++)
					{
						travScan[i] = { x, y };				
						i++;
					}
				}
				else
				{
					for(int32 x = TBlockSize - 1; x >= 0; x--)
					{
						travScan[i] = { x, y };				
						i++;
					}
				}
			}
			
			return travScan;			
		}
public:
		static constexpr TStaticArray<FIntPoint, TBlockSize * TBlockSize> Values = []() {
			switch (TscanIdx)
			{
				default:
				case 0:
					return MakeUpperRightDiagonal();
				case 1:
					return MakeHorizontal();				
				case 2:
					return MakeVertical();				
				case 3:
					return MakeTraverse();
			}
		};
	};

	struct scaling_list_data_t : FBitstreamSegment
	{
		TStaticArray<TStaticArray<U<1>, 6>, 4> scaling_list_pred_mode_flag;
		TStaticArray<TStaticArray<UE, 6>, 4> scaling_list_pred_matrix_id_delta;
		TStaticArray<TStaticArray<SE, 6>, 4> scaling_list_dc_coef_minus8;
		uint8 ScalingList0[6][16];
		uint8 ScalingList1to3[3][6][64];

        inline void Parse(FBitstreamReader& Bitstream);
	};

	struct FH265ChromaInfo
	{

	};
	
	const scaling_list_data_t DefaultScalingList = {
		{},
		{},
		{},
		{},
		{
			{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16},
			{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16},
			{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16},
			{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16},
			{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16},
			{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16}
		},
		{
			{
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 16, 17, 16, 17, 18, 17, 18, 18, 17, 18, 21, 19, 20, 21, 20, 19, 21, 24, 22, 22, 24, 24, 22, 22, 24, 25, 25, 27, 30, 27, 25, 25, 29, 31, 35, 35, 31, 29, 36, 41, 44, 41, 36, 47, 54, 54, 47, 65, 70, 65, 88, 88, 115},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 16, 17, 16, 17, 18, 17, 18, 18, 17, 18, 21, 19, 20, 21, 20, 19, 21, 24, 22, 22, 24, 24, 22, 22, 24, 25, 25, 27, 30, 27, 25, 25, 29, 31, 35, 35, 31, 29, 36, 41, 44, 41, 36, 47, 54, 54, 47, 65, 70, 65, 88, 88, 115},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 17, 17, 17, 17, 18, 18, 18, 18, 18, 18, 20, 20, 20, 20, 20, 20, 20, 24, 24, 24, 24, 24, 24, 24, 24, 25, 25, 25, 25, 25, 25, 25, 28, 28, 28, 28, 28, 28, 33, 33, 33, 33, 33, 41, 41, 41, 41, 54, 54, 54, 71, 71, 91},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 17, 17, 17, 17, 18, 18, 18, 18, 18, 18, 20, 20, 20, 20, 20, 20, 20, 24, 24, 24, 24, 24, 24, 24, 24, 25, 25, 25, 25, 25, 25, 25, 28, 28, 28, 28, 28, 28, 33, 33, 33, 33, 33, 41, 41, 41, 41, 54, 54, 54, 71, 71, 91},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 17, 17, 17, 17, 18, 18, 18, 18, 18, 18, 20, 20, 20, 20, 20, 20, 20, 24, 24, 24, 24, 24, 24, 24, 24, 25, 25, 25, 25, 25, 25, 25, 28, 28, 28, 28, 28, 28, 33, 33, 33, 33, 33, 41, 41, 41, 41, 54, 54, 54, 71, 71, 91},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 17, 17, 17, 17, 18, 18, 18, 18, 18, 18, 20, 20, 20, 20, 20, 20, 20, 24, 24, 24, 24, 24, 24, 24, 24, 25, 25, 25, 25, 25, 25, 25, 28, 28, 28, 28, 28, 28, 33, 33, 33, 33, 33, 41, 41, 41, 41, 54, 54, 54, 71, 71, 91},
			},
			{
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 16, 17, 16, 17, 18, 17, 18, 18, 17, 18, 21, 19, 20, 21, 20, 19, 21, 24, 22, 22, 24, 24, 22, 22, 24, 25, 25, 27, 30, 27, 25, 25, 29, 31, 35, 35, 31, 29, 36, 41, 44, 41, 36, 47, 54, 54, 47, 65, 70, 65, 88, 88, 115},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 16, 17, 16, 17, 18, 17, 18, 18, 17, 18, 21, 19, 20, 21, 20, 19, 21, 24, 22, 22, 24, 24, 22, 22, 24, 25, 25, 27, 30, 27, 25, 25, 29, 31, 35, 35, 31, 29, 36, 41, 44, 41, 36, 47, 54, 54, 47, 65, 70, 65, 88, 88, 115},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 17, 17, 17, 17, 18, 18, 18, 18, 18, 18, 20, 20, 20, 20, 20, 20, 20, 24, 24, 24, 24, 24, 24, 24, 24, 25, 25, 25, 25, 25, 25, 25, 28, 28, 28, 28, 28, 28, 33, 33, 33, 33, 33, 41, 41, 41, 41, 54, 54, 54, 71, 71, 91},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 17, 17, 17, 17, 18, 18, 18, 18, 18, 18, 20, 20, 20, 20, 20, 20, 20, 24, 24, 24, 24, 24, 24, 24, 24, 25, 25, 25, 25, 25, 25, 25, 28, 28, 28, 28, 28, 28, 33, 33, 33, 33, 33, 41, 41, 41, 41, 54, 54, 54, 71, 71, 91},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 17, 17, 17, 17, 18, 18, 18, 18, 18, 18, 20, 20, 20, 20, 20, 20, 20, 24, 24, 24, 24, 24, 24, 24, 24, 25, 25, 25, 25, 25, 25, 25, 28, 28, 28, 28, 28, 28, 33, 33, 33, 33, 33, 41, 41, 41, 41, 54, 54, 54, 71, 71, 91},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 17, 17, 17, 17, 18, 18, 18, 18, 18, 18, 20, 20, 20, 20, 20, 20, 20, 24, 24, 24, 24, 24, 24, 24, 24, 25, 25, 25, 25, 25, 25, 25, 28, 28, 28, 28, 28, 28, 33, 33, 33, 33, 33, 41, 41, 41, 41, 54, 54, 54, 71, 71, 91},
			},
			{				
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 16, 17, 16, 17, 18, 17, 18, 18, 17, 18, 21, 19, 20, 21, 20, 19, 21, 24, 22, 22, 24, 24, 22, 22, 24, 25, 25, 27, 30, 27, 25, 25, 29, 31, 35, 35, 31, 29, 36, 41, 44, 41, 36, 47, 54, 54, 47, 65, 70, 65, 88, 88, 115},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 16, 17, 16, 17, 18, 17, 18, 18, 17, 18, 21, 19, 20, 21, 20, 19, 21, 24, 22, 22, 24, 24, 22, 22, 24, 25, 25, 27, 30, 27, 25, 25, 29, 31, 35, 35, 31, 29, 36, 41, 44, 41, 36, 47, 54, 54, 47, 65, 70, 65, 88, 88, 115},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 17, 17, 17, 17, 18, 18, 18, 18, 18, 18, 20, 20, 20, 20, 20, 20, 20, 24, 24, 24, 24, 24, 24, 24, 24, 25, 25, 25, 25, 25, 25, 25, 28, 28, 28, 28, 28, 28, 33, 33, 33, 33, 33, 41, 41, 41, 41, 54, 54, 54, 71, 71, 91},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 17, 17, 17, 17, 18, 18, 18, 18, 18, 18, 20, 20, 20, 20, 20, 20, 20, 24, 24, 24, 24, 24, 24, 24, 24, 25, 25, 25, 25, 25, 25, 25, 28, 28, 28, 28, 28, 28, 33, 33, 33, 33, 33, 41, 41, 41, 41, 54, 54, 54, 71, 71, 91},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 17, 17, 17, 17, 18, 18, 18, 18, 18, 18, 20, 20, 20, 20, 20, 20, 20, 24, 24, 24, 24, 24, 24, 24, 24, 25, 25, 25, 25, 25, 25, 25, 28, 28, 28, 28, 28, 28, 33, 33, 33, 33, 33, 41, 41, 41, 41, 54, 54, 54, 71, 71, 91},
				{16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 17, 17, 17, 17, 18, 18, 18, 18, 18, 18, 20, 20, 20, 20, 20, 20, 20, 24, 24, 24, 24, 24, 24, 24, 24, 25, 25, 25, 25, 25, 25, 25, 28, 28, 28, 28, 28, 28, 33, 33, 33, 33, 33, 41, 41, 41, 41, 54, 54, 54, 71, 71, 91},
			}
		}
	};

	struct short_term_ref_pic_set_t : FBitstreamSegment
	{
		U<1> inter_ref_pic_set_prediction_flag = 0;
		UE delta_idx_minus1 = 0;
		U<1> delta_rps_sign;
		UE abs_delta_rps_minus1;
		U<1>* used_by_curr_pic_flags;
		U<1>* use_delta_flags;

		UE num_negative_pics = 0;
		UE num_positive_pics = 0;
		UE* delta_poc_s0_minus1s;
		U<1>* used_by_curr_pic_s0_flags;
		UE* delta_poc_s1_minus1s;
		U<1>* used_by_curr_pic_s1_flags;

		// Derived Values (specification usage order)
		uint8 NumPositivePics = 0;
		TStaticArray<int32, 16> DeltaPocS1;
		uint8 NumNegativePics = 0;
		TStaticArray<int32, 16> DeltaPocS0;
		uint8 NumDeltaPocs = 0;
		TStaticArray<uint8, 16> UsedByCurrPicS0;
		TStaticArray<uint8, 16> UsedByCurrPicS1;

        inline void Parse(uint8 const& stRpsIdx, TArray<short_term_ref_pic_set_t> const& short_term_ref_pic_sets, FBitstreamReader& Bitstream);
		inline void CalculateValues(uint8 const& stRpsIdx, TArray<short_term_ref_pic_set_t>& st_ref_pic_set);
	};

	struct SPS_t : public FNalu
	{
		U<4> sps_video_parameter_set_id;
		U<3> sps_max_sub_layers_minus1;
		U<1> sps_temporal_id_nesting_flag;
		profile_tier_level_t profile_tier_level;
		UE sps_seq_parameter_set_id;
		UE chroma_format_idc;
		U<1> separate_colour_plane_flag;
		UE ChromaArrayType;
		UE pic_width_in_luma_samples;
		UE pic_height_in_luma_samples;
		U<1> conformance_window_flag;

		struct conf_win_t : public FBitstreamSegment
		{
			UE left_offset;
			UE right_offset;
			UE top_offset;
			UE bottom_offset;
		} conf_win;

		UE bit_depth_luma_minus8;
		UE bit_depth_chroma_minus8;
		UE log2_max_pic_order_cnt_lsb_minus4;

		U<1> sps_sub_layer_ordering_info_present_flag;

		struct sub_layer_ordering_info_t : public FBitstreamSegment
		{
			UE sps_max_dec_pic_buffering_minus1;
			UE sps_max_num_reorder_pics;
			UE sps_max_latency_increase_plus1;
		};

		TArray<sub_layer_ordering_info_t> sub_layer_ordering_infos;

		UE log2_min_luma_coding_block_size_minus3;
		UE log2_diff_max_min_luma_coding_block_size;
		UE log2_min_luma_transform_block_size_minus2;
		UE log2_diff_max_min_luma_transform_block_size;
		UE max_transform_hierarchy_depth_inter;
		UE max_transform_hierarchy_depth_intra;

		U<1> scaling_list_enabled_flag;
		U<1> sps_scaling_list_data_present_flag = 0;
		scaling_list_data_t scaling_list_data;

		U<1> amp_enabled_flag;
		U<1> sample_adaptive_offset_enabled_flag;

		U<1> pcm_enabled_flag;
		U<4> pcm_sample_bit_depth_luma_minus1;
		U<4> pcm_sample_bit_depth_chroma_minus1;
		UE log2_min_pcm_luma_coding_block_size_minus3;
		UE log2_diff_max_min_pcm_luma_coding_block_size;
		U<1> pcm_loop_filter_disabled_flag = 0;

		UE num_short_term_ref_pic_sets;
		mutable TArray<short_term_ref_pic_set_t> short_term_ref_pic_sets;

		U<1> long_term_ref_pics_present_flag;
		UE num_long_term_ref_pics_sps;
        TArray<U<>> lt_ref_pic_poc_lsb_sps = {};
        TArray<U<1>> used_by_curr_pic_lt_sps_flag;

		U<1> sps_temporal_mvp_enabled_flag;
		U<1> strong_intra_smoothing_enabled_flag;

		U<1> vui_parameters_present_flag;
		struct vui_parameters_t : public FBitstreamSegment
		{
			U<1> aspect_ratio_info_present_flag = 0;
			U<8> aspect_ratio_idc; // TODO add a enum for this
			U<16> sar_width;
			U<16> sar_height;

			U<1> overscan_info_present_flag;
			U<1> overscan_appropriate_flag;

			U<1> video_signal_type_present_flag;
			U<3> video_format;
			U<1> video_full_range_flag;
			U<1> colour_description_present_flag;

			U<8> colour_primaries;
			U<8> transfer_characteristics;
			U<8> matrix_coeffs;

			U<1> chroma_loc_info_present_flag;
			UE chroma_sample_loc_type_top_field;
			UE chroma_sample_loc_type_bottom_field;

			U<1> neutral_chroma_indication_flag;
			U<1> field_seq_flag;
			U<1> frame_field_info_present_flag;
			U<1> default_display_window_flag;

			UE def_disp_win_left_offset;
			UE def_disp_win_right_offset;
			UE def_disp_win_top_offset;
			UE def_disp_win_bottom_offset;

			U<1> vui_timing_info_present_flag;
			U<32> vui_num_units_in_tick;
			U<32> vui_time_scale;
			U<1> vui_poc_proportional_to_timing_flag;
			UE vui_num_ticks_poc_diff_one_minus1;
			U<1> vui_hrd_parameters_present_flag;
			hrd_parameters_t hrd_parameters;

			U<1> bitstream_restriction_flag;

			U<1> tiles_fixed_structure_flag;
			U<1> motion_vectors_over_pic_boundaries_flag;
			U<1> restricted_ref_pic_lists_flag;
			UE min_spatial_segmentation_idc;
			UE max_bytes_per_pic_denom;
			UE max_bits_per_min_cu_denom;
			UE log2_max_mv_length_horizontal;
			UE log2_max_mv_length_vertical;

            inline void Parse(FBitstreamReader& Bitstream, uint8 const& sps_max_sub_layers_minus1);
		} vui_parameters;

		U<1> sps_extension_present_flag;

		U<1> sps_range_extension_flag = 0;
		U<1> sps_multilayer_extension_flag = 0;
		U<1> sps_3d_extension_flag = 0;
		U<1> sps_scc_extension_flag = 0;
		U<4> sps_extension_4bits = 0;

		struct sps_range_extension_t : public FBitstreamSegment
		{
			U<1> transform_skip_rotation_enabled_flag;
			U<1> transform_skip_context_enabled_flag;
			U<1> implicit_rdpcm_enabled_flag;
			U<1> explicit_rdpcm_enabled_flag;
			U<1> extended_precision_processing_flag;
			U<1> intra_smoothing_disabled_flag;
			U<1> high_precision_offsets_enabled_flag;
			U<1> persistent_rice_adaptation_enabled_flag;
			U<1> cabac_bypass_alignment_enabled_flag;

            inline void Parse(FBitstreamReader& Bitstream);
		} sps_range_extension;

		struct sps_multilayer_extension_t : public FBitstreamSegment
		{
			// TODO (aidan)
            inline void Parse(FBitstreamReader& Bitstream) { unimplemented(); }
		} sps_multilayer_extension;

		struct sps_3d_extension_t : public FBitstreamSegment
		{
			// TODO (aidan)
            inline void Parse(FBitstreamReader& Bitstream) { unimplemented(); }
		} sps_3d_extension;

		struct sps_scc_extension_t : public FBitstreamSegment
		{
			U<1> sps_curr_pic_ref_enabled_flag;
			U<1> palette_mode_enabled_flag;
			UE palette_max_size;
			UE delta_palette_max_predictor_size;
			U<1> sps_palette_predictor_initializers_present_flag;
			UE sps_num_palette_predictor_initializers_minus1;
            TArray<TArray<U<>>> sps_palette_predictor_initializers = {};
			U<2> motion_vector_resolution_control_idc;
			U<1> intra_boundary_filtering_disabled_flag;

            inline void Parse(uint32 const& in_chroma_format_idc, uint32 const& in_bit_depth_chroma_minus8, FBitstreamReader& Bitstream);
		} sps_scc_extension;

		// Members derived from bitstream
		uint32 MaxDpbSize;
	};

    FAVResult ParseSPS(FBitstreamReader& Bitstream, FNaluH265 const& InNaluInfo, TMap<uint32, VPS_t> const& InMapVPS, TMap<uint32, SPS_t>& OutMapSPS);


	struct PPS_t : public FNalu
	{
		UE pps_pic_parameter_set_id;
		UE pps_seq_parameter_set_id;
		U<1> dependent_slice_segments_enabled_flag;
		U<1> output_flag_present_flag;
		U<3> num_extra_slice_header_bits;
		U<1> sign_data_hiding_enabled_flag;
		U<1> cabac_init_present_flag;
		UE num_ref_idx_l0_default_active_minus1;
		UE num_ref_idx_l1_default_active_minus1;
		SE init_qp_minus26;
		U<1> constrained_intra_pred_flag;
		U<1> transform_skip_enabled_flag;
		U<1> cu_qp_delta_enabled_flag;
		UE diff_cu_qp_delta_depth;
		SE pps_cb_qp_offset;
		SE pps_cr_qp_offset;
		U<1> pps_slice_chroma_qp_offsets_present_flag;
		U<1> weighted_pred_flag;
		U<1> weighted_bipred_flag;
		U<1> transquant_bypass_enabled_flag;
		U<1> tiles_enabled_flag;
		U<1> entropy_coding_sync_enabled_flag;
		UE num_tile_columns_minus1 = 0;
		UE num_tile_rows_minus1 = 0;
		U<1> uniform_spacing_flag = 1; // NVParser seems to default this to 1
		TArray<UE> column_width_minus1;
		TArray<UE> row_height_minus1;
		U<1> loop_filter_across_tiles_enabled_flag = 1;
		U<1> pps_loop_filter_across_slices_enabled_flag;
		U<1> deblocking_filter_control_present_flag;
		U<1> deblocking_filter_override_enabled_flag = 0;
		U<1> pps_deblocking_filter_disabled_flag = 0;
		SE pps_beta_offset_div2;
		SE pps_tc_offset_div2;
		U<1> pps_scaling_list_data_present_flag;
		scaling_list_data_t scaling_list_data;
		U<1> lists_modification_present_flag;
		UE log2_parallel_merge_level_minus2;
		U<1> slice_segment_header_extension_present_flag;
		U<1> pps_extension_present_flag;
		U<1> pps_range_extension_flag = 0;
		U<1> pps_multilayer_extension_flag = 0;
		U<1> pps_3d_extension_flag = 0;
		U<1> pps_scc_extension_flag = 0;
		U<4> pps_extension_4bits = 0;

		struct pps_range_extension_t
		{
			UE log2_max_transform_skip_block_size_minus2;
			U<1> cross_component_prediction_enabled_flag;
			U<1> chroma_qp_offset_list_enabled_flag;
			UE diff_cu_chroma_qp_offset_depth;
			UE chroma_qp_offset_list_len_minus1;
			TArray<SE> cb_qp_offset_list;
			TArray<SE> cr_qp_offset_list;
			UE log2_sao_offset_scale_luma;
			UE log2_sao_offset_scale_chroma;

            inline void Parse(uint8 const& in_transform_skip_enabled_flag, FBitstreamReader& Bitstream);
		} pps_range_extension;

		struct pps_multilayer_extension_t
		{
            inline void Parse(FBitstreamReader& Bitstream) { /*unimplemented();*/ }
		} pps_multilayer_extension;

		struct pps_3d_extension_t
		{
            inline void Parse(FBitstreamReader& Bitstream) { /*unimplemented();*/ }
		} pps_3d_extension;

		struct pps_scc_extension_t
		{
			U<1> pps_curr_pic_ref_enabled_flag;
			U<1> residual_adaptive_colour_transform_enabled_flag;
			U<1> pps_slice_act_qp_offsets_present_flag;
			SE pps_act_y_qp_offset_plus5;
			SE pps_act_cb_qp_offset_plus5;
			SE pps_act_cr_qp_offset_plus3;
			U<1> pps_palette_predictor_initializers_present_flag;
			UE pps_num_palette_predictor_initializers;
			U<1> monochrome_palette_flag;
			UE luma_bit_depth_entry_minus8;
			UE chroma_bit_depth_entry_minus8;
            TArray<TArray<U<>>> pps_palette_predictor_initializer = {};

            inline void Parse(FBitstreamReader& Bitstream);
		} pps_scc_extension;
	};

    FAVResult ParsePPS(FBitstreamReader& Bitstream, FNaluH265 const& InNaluInfo, TMap<uint32, VPS_t> const& InMapVPS, TMap<uint32, SPS_t> const& InMapSPS, TMap<uint32, PPS_t>& OutMapPPS);

	enum class EH265SliceType : uint8
	{
		B = 0,
		P = 1,
		I = 2
	};

	struct Slice_t : public FNalu
	{
		// Members read from bitstream
		U<1> first_slice_segment_in_pic_flag;
		U<1> no_output_of_prior_pics_flag;
		UE slice_pic_parameter_set_id;

		U<1> dependent_slice_segment_flag = 0;
		U<> slice_segment_address = 0;
		UnsignedExpGolomb<EH265SliceType> slice_type;
		U<1> pic_output_flag = 1;
		U<2> colour_plane_id;
		U<> slice_pic_order_cnt_lsb = 0;
		U<1> short_term_ref_pic_set_sps_flag;
		short_term_ref_pic_set_t short_term_ref_pic_set;
		U<> short_term_ref_pic_set_idx = 0;
		UE num_long_term_sps = 0;
		UE num_long_term_pics = 0;
        U<>* lt_idx_sps = nullptr;
        U<>* poc_lsb_lt = nullptr;
		U<1>* used_by_curr_pic_lt_flag = nullptr;
	    U<1>* delta_poc_msb_present_flag = nullptr;
		UE* delta_poc_msb_cycle_lt = nullptr;
		U<1> slice_temporal_mvp_enabled_flag = 0;

		U<1> slice_sao_luma_flag = 0;
		U<1> slice_sao_chroma_flag = 0;

		U<1> num_ref_idx_active_override_flag;
		UE num_ref_idx_l0_active_minus1 = 0;
		UE num_ref_idx_l1_active_minus1 = 0;
		struct ref_pic_list_modification_t
		{
			U<1> ref_pic_list_modification_flag_l0 = 0;
            U<>* list_entry_l0 = nullptr;
			U<1> ref_pic_list_modification_flag_l1 = 0;
            U<>* list_entry_l1 = nullptr;

            inline void Parse(uint8 const& in_num_ref_idx_l0_active_minus1, uint8 const& in_num_ref_idx_l1_active_minus1, uint32 const& InNumPicTotalCurr, EH265SliceType const& SliceType, FBitstreamReader& BitStream);
		} ref_pic_list_modification;
		U<1> mvd_l1_zero_flag = 0;
		U<1> cabac_init_flag = 0;
		U<1> collocated_from_l0_flag = 1;
		UE collocated_ref_idx = 0;
		struct pred_weight_table_t
		{
			UE luma_log2_weight_denom;
			SE delta_chroma_log2_weight_denom = 0;

			TStaticArray<U<1>, 15> luma_weight_l0_flag;
			TStaticArray<U<1>, 15> chroma_weight_l0_flag;
			TStaticArray<SE, 15> delta_luma_weight_l0;
			TStaticArray<SE, 15> luma_offset_l0;
			TStaticArray<TStaticArray<SE, 2>, 15> delta_chroma_weight_l0;
			TStaticArray<TStaticArray<SE, 2>, 15> delta_chroma_offset_l0;

			TStaticArray<U<1>, 15> luma_weight_l1_flag;
			TStaticArray<U<1>, 15> chroma_weight_l1_flag;
			TStaticArray<SE, 15> delta_luma_weight_l1;
			TStaticArray<SE, 15> luma_offset_l1;
			TStaticArray<TStaticArray<SE, 2>, 15> delta_chroma_weight_l1;
			TStaticArray<TStaticArray<SE, 2>, 15> delta_chroma_offset_l1;

            inline void Parse(const uint8& ChromaArrayType, FNaluH265 const& InNaluInfo, Slice_t const& CurrentSlice, FBitstreamReader& BitStream);
        } pred_weight_table;

		UE five_minus_max_num_merge_cand;
		U<1> use_integer_mv_flag;

		SE slice_qp_delta;

		SE slice_cb_qp_offset = 0;
		SE slice_cr_qp_offset = 0;

		SE slice_act_y_qp_offset;
		SE slice_act_cb_qp_offset;
		SE slice_act_cr_qp_offset;

		U<1> cu_chroma_qp_offset_enabled_flag = 0;

		U<1> deblocking_filter_override_flag = 0;

		U<1> slice_deblocking_filter_disabled_flag;
		SE slice_beta_offset_div2;
		SE slice_tc_offset_div2;

		U<1> slice_loop_filter_across_slices_enabled_flag;

		UE num_entry_point_offsets = 0;
		UE offset_len_minus1;
        U<>* entry_point_offset_minus1;

		UE slice_segment_header_extension_length = 0;
		U<8>* slice_segment_header_extension_data_byte;

		uint32 CurrPicIdx = 0;
		uint32 CurrPicOrderCntVal = 0;
		uint32 CurrRpsIdx = 0;
	};

    FAVResult ParseSliceHeader(FBitstreamReader& Bitstream, FNaluH265 const& InNaluInfo, TMap<uint32, VPS_t> const& InMapVPS, TMap<uint32, SPS_t> const& InMapSPS, TMap<uint32, PPS_t> const& InMapPPS, Slice_t& OutSlice);


	struct SEI_t : public FNalu
	{
	};

    FAVResult ParseSEI(FBitstreamReader& Bitstream, FNaluH265 const& InNaluInfo, SEI_t& OutSEI);

} // namespace UE::AVCodecCore::H265
