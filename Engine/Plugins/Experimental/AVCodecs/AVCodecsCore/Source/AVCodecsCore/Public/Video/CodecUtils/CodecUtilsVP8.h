// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Utils/BitstreamReader.h"
#include "AVResult.h"

namespace UE::AVCodecCore::VP8
{
	enum class EDenoiserState : uint32
	{
		DenoiserOff,
		DenoiserOnYOnly,
		DenoiserOnYUV,
		DenoiserOnYUVAggressive,
		// Adaptive mode defaults to DenoiserOnYUV on key frame, but may switch
		// to DenoiserOnYUVAggressive based on a computed noise metric.
		DenoiserOnAdaptive
	};

	enum class EBufferFlags : uint8
	{
		None = 0b00,
		Reference = 0b01,
		Update = 0b10,
		ReferenceAndUpdate = Reference | Update,
	};

	ENUM_CLASS_FLAGS(EBufferFlags);

	enum class EBufferType : int
	{
		Last = 0b00,
		Golden = 0b01,
		Arf = 0b10,
		Count
	};

	ENUM_CLASS_FLAGS(EBufferType);

	struct FVP8FrameConfig
	{
		static FVP8FrameConfig GetIntraFrameConfig()
		{
			FVP8FrameConfig FrameConfig = FVP8FrameConfig(EBufferFlags::Update, EBufferFlags::Update, EBufferFlags::Update);
			return FrameConfig;
		}

		FVP8FrameConfig()
			: LastBufferFlags(EBufferFlags::None)
			, GoldenBufferFlags(EBufferFlags::None)
			, ArfBufferFlags(EBufferFlags::None)
		{
		}

		FVP8FrameConfig(EBufferFlags Last, EBufferFlags Golden, EBufferFlags Arf)
			: LastBufferFlags(Last)
			, GoldenBufferFlags(Golden)
			, ArfBufferFlags(Arf)
		{
		}

		bool References(EBufferType Buffer) const
		{
			switch (Buffer)
			{
				case EBufferType::Last:
					return static_cast<uint8>(LastBufferFlags & EBufferFlags::Reference) != 0;
				case EBufferType::Golden:
					return static_cast<uint8>(GoldenBufferFlags & EBufferFlags::Reference) != 0;
				case EBufferType::Arf:
					return static_cast<uint8>(ArfBufferFlags & EBufferFlags::Reference) != 0;
				case EBufferType::Count:
					break;
			}
			return false;
		}

		bool Updates(EBufferType Buffer) const
		{
			switch (Buffer)
			{
				case EBufferType::Last:
					return static_cast<uint8>(LastBufferFlags & EBufferFlags::Update) != 0;
				case EBufferType::Golden:
					return static_cast<uint8>(GoldenBufferFlags & EBufferFlags::Update) != 0;
				case EBufferType::Arf:
					return static_cast<uint8>(ArfBufferFlags & EBufferFlags::Update) != 0;
				case EBufferType::Count:
					break;
			}
			return false;
		}

		bool IntraFrame() const
		{
			// Intra frames do not reference any buffers, and update all buffers.
			return LastBufferFlags == EBufferFlags::Update && GoldenBufferFlags == EBufferFlags::Update && ArfBufferFlags == EBufferFlags::Update;
		}

		EBufferFlags LastBufferFlags;
		EBufferFlags GoldenBufferFlags;
		EBufferFlags ArfBufferFlags;		
	};
} // namespace UE::AVCodecCore::VP8
