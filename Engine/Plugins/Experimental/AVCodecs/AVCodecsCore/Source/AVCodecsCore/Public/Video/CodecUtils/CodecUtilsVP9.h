// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Utils/BitstreamReader.h"
#include "AVResult.h"

enum class EScalabilityMode : uint8;

namespace UE::AVCodecCore::VP9
{
	constexpr int16	 MaxOneBytePictureId = 0x7F;   // 7 bits
	constexpr int16	 MaxTwoBytePictureId = 0x7FFF; // 15 bits
	constexpr uint8	 NoSpatialIdx = 0xFF;
	constexpr uint8	 NoGofIdx = 0xFF;
	constexpr uint8	 NumBuffers = 8;
	constexpr size_t MaxRefPics = 3;
	constexpr size_t MaxFramesInGof = 0xFF; // 8 bits
	constexpr size_t MaxNumberOfSpatialLayers = 8;

	constexpr size_t MinSpatialLayerLongSideLength = 240;
	constexpr size_t MinSpatialLayerShortSideLength = 135;

	enum class EProfile : uint8
	{
		Profile0,
		Profile1,
		Profile2,
		Profile3
	};

	enum class EInterLayerPrediction : uint8
	{
		Off,
		On,
		OnKeyPicture
	};

	enum class ETemporalStructureMode : uint8
	{
		TemporalStructureMode1, // 1 temporal layer structure - i.e., IPPP...
		TemporalStructureMode2, // 2 temporal layers 01...
		TemporalStructureMode3, // 3 temporal layers 0212...
	};

	struct FGroupOfFramesInfo
	{
		void SetGofInfo(ETemporalStructureMode tm)
		{
			switch (tm)
			{
				case ETemporalStructureMode::TemporalStructureMode1:
					NumFramesInGof = 1;
					TemporalIdx[0] = 0;
					TemporalUpSwitch[0] = true;
					NumRefPics[0] = 1;
					PidDiff[0][0] = 1;
					break;
				case ETemporalStructureMode::TemporalStructureMode2:
					NumFramesInGof = 2;
					TemporalIdx[0] = 0;
					TemporalUpSwitch[0] = true;
					NumRefPics[0] = 1;
					PidDiff[0][0] = 2;

					TemporalIdx[1] = 1;
					TemporalUpSwitch[1] = true;
					NumRefPics[1] = 1;
					PidDiff[1][0] = 1;
					break;
				case ETemporalStructureMode::TemporalStructureMode3:
					NumFramesInGof = 4;
					TemporalIdx[0] = 0;
					TemporalUpSwitch[0] = true;
					NumRefPics[0] = 1;
					PidDiff[0][0] = 4;

					TemporalIdx[1] = 2;
					TemporalUpSwitch[1] = true;
					NumRefPics[1] = 1;
					PidDiff[1][0] = 1;

					TemporalIdx[2] = 1;
					TemporalUpSwitch[2] = true;
					NumRefPics[2] = 1;
					PidDiff[2][0] = 2;

					TemporalIdx[3] = 2;
					TemporalUpSwitch[3] = true;
					NumRefPics[3] = 1;
					PidDiff[3][0] = 1;
					break;
				default:
					checkNoEntry();
			}
		}

		size_t NumFramesInGof;
		uint8  TemporalIdx[MaxFramesInGof];
		bool   TemporalUpSwitch[MaxFramesInGof];
		uint8  NumRefPics[MaxFramesInGof];
		uint8  PidDiff[MaxFramesInGof][MaxRefPics];
		uint16 PidStart;
	};

	enum class EBitDepth : uint8
	{
		k8Bit = 8,
		k10Bit = 10,
		k12Bit = 12,
	};

	enum class EColorSpace : uint8
	{
		UNKNOWN = 0,   // Unknown (in this case the color space must be signaled
					   // outside the VP9 bitstream).
		BT_601 = 1,	   // CS_BT_601 Rec. ITU-R BT.601-7
		BT_709 = 2,	   // Rec. ITU-R BT.709-6
		SMPTE_170 = 3, // SMPTE-170
		SMPTE_240 = 4, // SMPTE-240
		BT_2020 = 5,   // Rec. ITU-R BT.2020-2
		RESERVED = 6,  // Reserved
		RGB = 7,	   // sRGB (IEC 61966-2-1)
	};

	enum class EColorRange : uint8
	{
		Studio, // Studio swing:
				// For BitDepth equals 8:
				//     Y is between 16 and 235 inclusive.
				//     U and V are between 16 and 240 inclusive.
				// For BitDepth equals 10:
				//     Y is between 64 and 940 inclusive.
				//     U and V are between 64 and 960 inclusive.
				// For BitDepth equals 12:
				//     Y is between 256 and 3760.
				//     U and V are between 256 and 3840 inclusive.
		Full	// Full swing; no restriction on Y, U, V values.
	};

	enum class ESubSampling : uint8
	{
		k444,
		k440,
		k422,
		k420,
	};

	enum class EReferenceFrame : int
	{
		None = -1,
		Intra = 0,
		Last = 1,
		Golden = 2,
		Altref = 3
	};

	enum class EInterpolationFilter : uint8
	{
		EightTap = 0,
		EightTapSmooth = 1,
		EightTapSharp = 2,
		Bilinear = 3,
		Switchable = 4
	};

	struct Header_t : public FNalu
	{
		U<2> frame_marker = 0;
		U<1> profile_low_bit = 0;
		U<1> profile_high_bit = 0;
		U<1> show_existing_frame = 0;
		U<3> frame_to_show_map_idx = 0;
		U<1> frame_type = 0;
		U<1> show_frame = 0;
		U<1> error_resilient_mode = 0;

		U<8> frame_sync_byte_0;
		U<8> frame_sync_byte_1;
		U<8> frame_sync_byte_2;

		U<1, EBitDepth> bit_depth;

		U<1> intra_only = 0;
		U<2> reset_frame_context = 0;

		U<1>			   ten_or_twelve_bit = 0;
		U<3, EColorSpace>  color_space;
		U<1, EColorRange>  color_range = EColorRange::Studio;
		U<2, ESubSampling> sub_sampling;

		U<16> frame_width_minus_1;
		U<16> frame_height_minus_1;

		// Pre-calculated helpers
		EProfile Profile = EProfile::Profile0;
		uint16		FrameWidth = 0;
		uint16		FrameHeight = 0;
	};

	FAVResult ParseHeader(FBitstreamReader& Bitstream, Header_t& OutHeader);

	FAVResult ParseFrameSyncCode(FBitstreamReader& Bitstream, Header_t& OutHeader);
	FAVResult ParseColorConfig(FBitstreamReader& Bitstream, Header_t& OutHeader);
	FAVResult ParseFrameSize(FBitstreamReader& Bitstream, Header_t& OutHeader);

	AVCODECSCORE_API EInterLayerPrediction ScalabilityModeToInterLayerPredMode(EScalabilityMode ScalabilityMode);

} // namespace UE::AVCodecCore::VP9
