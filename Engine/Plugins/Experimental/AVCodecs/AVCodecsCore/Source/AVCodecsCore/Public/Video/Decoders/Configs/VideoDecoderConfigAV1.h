// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoDecoder.h"
#include "Video/CodecUtils/CodecUtilsAV1.h"

/*
 * Configuration settings for AV1 decoders.
 */
struct FVideoDecoderConfigAV1 : public FVideoDecoderConfig
{
    FVideoDecoderConfigAV1(EAVPreset Preset = EAVPreset::Default)
		: FVideoDecoderConfig(Preset)
	{
	}
};

DECLARE_TYPEID(FVideoDecoderConfigAV1, AVCODECSCORE_API);
