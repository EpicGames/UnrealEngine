// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoDecoder.h"
#include "Video/CodecUtils/CodecUtilsH264.h"

#define UE_API AVCODECSCORE_API

/*
 * Configuration settings for H264 decoders.
 */
struct FVideoDecoderConfigH264 : public FVideoDecoderConfig
{
	TMap<uint32, UE::AVCodecCore::H264::SPS_t> SPS;
	TMap<uint32, UE::AVCodecCore::H264::PPS_t> PPS;
	TArray<UE::AVCodecCore::H264::SEI_t> SEI;

	FVideoDecoderConfigH264(EAVPreset Preset = EAVPreset::Default)
		: FVideoDecoderConfig(Preset)
	{
	}

	UE_API FAVResult Parse(FVideoPacket const& Packet, TArray<UE::AVCodecCore::H264::Slice_t>& OutSlices);

    UE_API TOptional<int> GetLastSliceQP(TArray<UE::AVCodecCore::H264::Slice_t>& Slices);
};

DECLARE_TYPEID(FVideoDecoderConfigH264, AVCODECSCORE_API);

#undef UE_API
