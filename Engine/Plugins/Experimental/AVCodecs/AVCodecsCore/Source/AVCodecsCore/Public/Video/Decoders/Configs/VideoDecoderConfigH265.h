// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoDecoder.h"
#include "Video/CodecUtils/CodecUtilsH265.h"

#define UE_API AVCODECSCORE_API

/*
 * Configuration settings for H265 decoders.
 */
struct FVideoDecoderConfigH265 : public FVideoDecoderConfig
{
    TMap<uint32, UE::AVCodecCore::H265::VPS_t> VPS;
	TMap<uint32, UE::AVCodecCore::H265::SPS_t> SPS;
	TMap<uint32, UE::AVCodecCore::H265::PPS_t> PPS;
	TArray<UE::AVCodecCore::H265::SEI_t> SEI;

	FVideoDecoderConfigH265(EAVPreset Preset = EAVPreset::Default)
		: FVideoDecoderConfig(Preset)
	{
	}

	UE_API FAVResult Parse(FVideoPacket const& Packet, TArray<UE::AVCodecCore::H265::Slice_t>& OutSlices);

    UE_API TOptional<int> GetLastSliceQP(TArray<UE::AVCodecCore::H265::Slice_t>& Slices);
};

DECLARE_TYPEID(FVideoDecoderConfigH265, AVCODECSCORE_API);

#undef UE_API
