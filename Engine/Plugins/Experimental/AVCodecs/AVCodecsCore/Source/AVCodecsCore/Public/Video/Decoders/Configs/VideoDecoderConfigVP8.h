// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoDecoder.h"

/*
 * Configuration settings for VP8 decoders.
 */
struct FVideoDecoderConfigVP8 : public FVideoDecoderConfig
{
	int32 NumberOfCores = 0;

	FVideoDecoderConfigVP8(EAVPreset Preset = EAVPreset::Default)
		: FVideoDecoderConfig(Preset)
	{
	}
};

DECLARE_TYPEID(FVideoDecoderConfigVP8, AVCODECSCORE_API);
