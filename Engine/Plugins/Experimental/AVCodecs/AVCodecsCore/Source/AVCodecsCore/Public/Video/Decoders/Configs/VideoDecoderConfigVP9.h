// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoDecoder.h"
#include "Video/CodecUtils/CodecUtilsVP9.h"

#define UE_API AVCODECSCORE_API

/*
 * Configuration settings for VP9 decoders.
 */
struct FVideoDecoderConfigVP9 : public FVideoDecoderConfig
{
	// Values are parsed from the bitstream during decoding
	uint32 MaxOutputWidth = 0;
	uint32 MaxOutputHeight = 0;
	
	int32  NumberOfCores = 0;

	FVideoDecoderConfigVP9(EAVPreset Preset = EAVPreset::Default)
		: FVideoDecoderConfig(Preset)
	{
	}

	UE_API FAVResult Parse(FVideoPacket const& Packet, UE::AVCodecCore::VP9::Header_t& OutHeader);
};

DECLARE_TYPEID(FVideoDecoderConfigVP9, AVCODECSCORE_API);

#undef UE_API
