// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoDecoder.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH265.h"

/**
 * Configuration-less decoder that detects the bitstream type and creates an appropriate child decoder that can handle it.
 */
template <typename TResource>
class TStreamVideoDecoder : public TVideoDecoder<TResource>
{
private:
	/**
	 * Whether this encoder is open and is or isn't initialized.
	 */
	uint8 bIsOpen : 1;
	
public:
	/**
	 * The child decoder, invalid until this decoder is initialized by sending through a packet.
	 */
	TSharedPtr<TVideoDecoder<TResource>> Child;

	TStreamVideoDecoder() = default;

	virtual bool IsOpen() const override
	{
		return bIsOpen;
	}

	/**
	 * @return True if this decoder has been initialized by sending through a packet.
	 */
	bool IsInitialized() const
	{
		return Child.IsValid();
	}

	virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice,TSharedRef<FAVInstance> const& NewInstance) override
	{
		if (!this->IsOpen())
		{
			FAVResult Result = TVideoDecoder<TResource>::Open(NewDevice, NewInstance);

			bIsOpen = Result;

			return Result;
		}

		return EAVResult::Error;
	}
	
	virtual void Close() override
	{
		if (this->IsOpen())
		{
			Child.Reset();

			bIsOpen = false;
		}
	}

	virtual FVideoDecoderConfig GetMinimalConfig() override
	{
		if (this->IsOpen())
		{
			return this->Child->GetMinimalConfig();
		}

		return FVideoDecoderConfig();
	}

	virtual void SetMinimalConfig(FVideoDecoderConfig const& MinimalConfig) override
	{
		if (this->IsOpen())
		{
			this->SetMinimalConfig(MinimalConfig);
		}
	}

	virtual FAVResult ApplyConfig() override
	{
		if (this->IsOpen())
		{
			return this->Child->ApplyConfig();
		}

		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"));
	}

	virtual FAVResult SendPacket(FVideoPacket const& Packet) override
	{
		if (this->IsOpen())
		{
			if (IsInitialized())
			{
				FAVResult const Result = Child->SendPacket(Packet);
				if (Result.Value >= EAVResult::Pending)
				{
					return Result;
				}
			}
			
			if (!IsInitialized())
			{
				TArray<UE::AVCodecCore::H264::Slice_t> H264Dummy;
				TArray<TSharedPtr<UE::AVCodecCore::H265::FNaluH265>> H265Dummy;
				
				if (FVideoDecoderConfigH264().Parse(this->GetInstance().ToSharedRef(), Packet, H264Dummy).Handle())
				{
					Child = FVideoDecoder::Create<TResource, FVideoDecoderConfigH264>(this->GetDevice().ToSharedRef(), this->GetInstance().ToSharedRef());
				}
				else if (FVideoDecoderConfigH265().Parse(this->GetInstance().ToSharedRef(), Packet, H265Dummy).Handle())
				{
					Child = FVideoDecoder::Create<TResource, FVideoDecoderConfigH265>(this->GetDevice().ToSharedRef(), this->GetInstance().ToSharedRef());
				}
				
				if (!Child.IsValid())
				{
					return FAVResult(EAVResult::ErrorCreating, TEXT("Could not create a decoder to support the bitstream"));
				}
			}
		
			return Child->SendPacket(Packet);
		}

		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"));
	}

	virtual FAVResult ReceiveFrame(TResolvableVideoResource<TResource>& InOutResource) override
	{
		if (this->IsInitialized())
		{
			return Child->ReceiveFrame(InOutResource);
		}

		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"));
	}
};
