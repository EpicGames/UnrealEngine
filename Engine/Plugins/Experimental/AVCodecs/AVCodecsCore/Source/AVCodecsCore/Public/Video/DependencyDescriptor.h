// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Math/IntPoint.h"

#define UE_API AVCODECSCORE_API

// Relationship of a frame to a Decode target.
enum class AVCODECSCORE_API EDecodeTargetIndication : uint8
{
	NotPresent = 0,	 // DecodeTargetInfo symbol '-'
	Discardable = 1, // DecodeTargetInfo symbol 'D'
	Switch = 2,		 // DecodeTargetInfo symbol 'S'
	Required = 3	 // DecodeTargetInfo symbol 'R'
};

class FFrameDependencyTemplate
{
public:
	// Setters are named briefly to chain them when building the template.
	UE_API FFrameDependencyTemplate& SpatialLayerId(int SpatialLayer);
	UE_API FFrameDependencyTemplate& TemporalLayerId(int TemporalLayer);
	UE_API FFrameDependencyTemplate& Dtis(const FString& DTIS);
	UE_API FFrameDependencyTemplate& FrameDiff(std::initializer_list<int32> Diffs);
	UE_API FFrameDependencyTemplate& ChainDiff(std::initializer_list<int32> Diffs);

	friend bool operator==(const FFrameDependencyTemplate& Lhs, const FFrameDependencyTemplate& Rhs)
	{
		return Lhs.SpatialId == Rhs.SpatialId
			&& Lhs.TemporalId == Rhs.TemporalId
			&& Lhs.DecodeTargetIndications == Rhs.DecodeTargetIndications
			&& Lhs.FrameDiffs == Rhs.FrameDiffs
			&& Lhs.ChainDiffs == Rhs.ChainDiffs;
	}

	int32							SpatialId = 0;
	int32							TemporalId = 0;
	TArray<EDecodeTargetIndication> DecodeTargetIndications;
	TArray<int32>					FrameDiffs;
	TArray<int32>					ChainDiffs;
};

class FFrameDependencyStructure
{
public:
	friend bool operator==(const FFrameDependencyStructure& Lhs, const FFrameDependencyStructure& Rhs)
	{
		return Lhs.NumDecodeTargets == Rhs.NumDecodeTargets
			&& Lhs.NumChains == Rhs.NumChains
			&& Lhs.DecodeTargetProtectedByChain == Rhs.DecodeTargetProtectedByChain
			&& Lhs.Resolutions == Rhs.Resolutions
			&& Lhs.Templates == Rhs.Templates;
	}

	int StructureId = 0;
	int NumDecodeTargets = 0;
	int NumChains = 0;
	// If chains are used (NumChains > 0), maps decode target index into index of
	// the chain protecting that target.
	TArray<int32>					 DecodeTargetProtectedByChain;
	TArray<FIntPoint>				 Resolutions;
	TArray<FFrameDependencyTemplate> Templates;
};

#undef UE_API
