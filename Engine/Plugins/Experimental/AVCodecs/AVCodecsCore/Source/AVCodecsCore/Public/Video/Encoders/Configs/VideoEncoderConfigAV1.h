// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoEncoder.h"
#include "Video/CodecUtils/CodecUtilsAV1.h"

/*
 * Configuration settings for AV1 encoders.
 */
struct FVideoEncoderConfigAV1 : public FVideoEncoderConfig
{
public:
	EAV1Profile Profile = EAV1Profile::Main;

	bool RepeatSeqHdr = false;

	uint32 IntraRefreshPeriodFrames = 0;
	uint32 IntraRefreshCountFrames = 0;
	uint32 BitDepthMinus8 = 0;

	FVideoEncoderConfigAV1(EAVPreset Preset = EAVPreset::Default)
		: FVideoEncoderConfig(Preset)
	{
	}
};

DECLARE_TYPEID(FVideoEncoderConfigAV1, AVCODECSCORE_API);