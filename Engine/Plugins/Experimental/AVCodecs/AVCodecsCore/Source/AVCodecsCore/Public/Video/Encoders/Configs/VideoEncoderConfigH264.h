// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoEncoder.h"
#include "Video/CodecUtils/CodecUtilsH264.h"

/*
 * Configuration settings for H264 encoders.
 */
struct FVideoEncoderConfigH264 : public FVideoEncoderConfig
{
public:
	EH264Profile Profile = EH264Profile::Main;
	EH264AdaptiveTransformMode AdaptiveTransformMode = EH264AdaptiveTransformMode::Auto;
	EH264EntropyCodingMode EntropyCodingMode = EH264EntropyCodingMode::Auto;

	bool RepeatSPSPPS = false;

	uint32 IntraRefreshPeriodFrames = 0;
	uint32 IntraRefreshCountFrames = 0;

	FVideoEncoderConfigH264(EAVPreset Preset = EAVPreset::Default)
		: FVideoEncoderConfig(Preset)
	{
	}
};

DECLARE_TYPEID(FVideoEncoderConfigH264, AVCODECSCORE_API);
