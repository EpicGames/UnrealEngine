// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoEncoder.h"
#include "Video/CodecUtils/CodecUtilsH265.h"

/*
 * Configuration settings for H265 encoders.
 */
struct FVideoEncoderConfigH265 : public FVideoEncoderConfig
{
	EH265Profile Profile = EH265Profile::Main;

	bool RepeatSPSPPS = false;
	
	int32 IntraRefreshPeriodFrames = 0;
	int32 IntraRefreshCountFrames = 0;

	FVideoEncoderConfigH265(EAVPreset Preset = EAVPreset::Default)
		: FVideoEncoderConfig(Preset)
	{
	}
};

DECLARE_TYPEID(FVideoEncoderConfigH265, AVCODECSCORE_API);
