// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoEncoder.h"
#include "Video/CodecUtils/CodecUtilsVP8.h"

/*
 * Configuration settings for VP8 encoders.
 */
struct FVideoEncoderConfigVP8 : public FVideoEncoderConfig
{
	int32 NumberOfCores = 0;
	bool  bDenoisingOn = false;

	FVideoEncoderConfigVP8(EAVPreset Preset = EAVPreset::Default)
		: FVideoEncoderConfig(Preset)
	{
	}
};

DECLARE_TYPEID(FVideoEncoderConfigVP8, AVCODECSCORE_API);
