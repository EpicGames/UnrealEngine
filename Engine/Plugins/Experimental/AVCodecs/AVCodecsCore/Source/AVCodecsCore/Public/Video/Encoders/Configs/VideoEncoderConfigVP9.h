// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoEncoder.h"
#include "Video/CodecUtils/CodecUtilsVP9.h"

using namespace UE::AVCodecCore::VP9;

/*
 * Configuration settings for VP9 encoders.
 */
struct FVideoEncoderConfigVP9 : public FVideoEncoderConfig
{
	int32					 NumberOfCores = 0;
	bool					 bDenoisingOn = false;
	bool					 bAdaptiveQpMode = false;
	bool					 bAutomaticResizeOn = false;
	bool					 bFlexibleMode = false;
	EInterLayerPrediction	 InterLayerPrediction = EInterLayerPrediction::Off;

	FVideoEncoderConfigVP9(EAVPreset Preset = EAVPreset::Default)
		: FVideoEncoderConfig(Preset)
	{
	}
};

DECLARE_TYPEID(FVideoEncoderConfigVP9, AVCODECSCORE_API);
