// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/Encoders/SVC/ScalabilityStructureFull.h"
#include "Video/Encoders/SVC/ScalabilityStructureKey.h"
#include "Video/Encoders/SVC/ScalabilityStructureL2T2KeyShift.h"
#include "Video/Encoders/SVC/ScalabilityStructureSimulcast.h"
#include "Video/Encoders/SVC/ScalableVideoController.h"
#include "Video/Encoders/SVC/ScalableVideoControllerNoLayering.h"
#include "Video/VideoEncoder.h"

AVCODECSCORE_API TUniquePtr<FScalableVideoController> CreateScalabilityStructure(EScalabilityMode Name);

// Returns description of the scalability structure identified by 'name',
// Return nullopt for unknown name.
AVCODECSCORE_API TOptional<FScalableVideoController::FStreamLayersConfig> ScalabilityStructureConfig(EScalabilityMode Name);