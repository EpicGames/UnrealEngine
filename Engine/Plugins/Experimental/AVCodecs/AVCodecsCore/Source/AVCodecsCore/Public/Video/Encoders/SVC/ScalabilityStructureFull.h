// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/GenericFrameInfo.h"
#include "Video/Encoders/SVC/ScalableVideoController.h"

#define UE_API AVCODECSCORE_API

class FScalabilityStructureFullSvc : public FScalableVideoController
{
public:
	UE_API FScalabilityStructureFullSvc(int NumSpatialLayers, int NumTemporalLayers, FIntFraction ResolutionFactor);
	~FScalabilityStructureFullSvc() override = default;

	UE_API virtual FStreamLayersConfig StreamConfig() const override;

	UE_API virtual TArray<FScalableVideoController::FLayerFrameConfig> NextFrameConfig(bool bRestart) override;
	UE_API virtual FGenericFrameInfo									OnEncodeDone(const FScalableVideoController::FLayerFrameConfig& Config) override;
	UE_API virtual void												OnRatesUpdated(const FVideoBitrateAllocation& Bitrates) override;

private:
	// NOTE: This enum name is duplicated throughout the ScalabilityStructure variants
	// While it's name is the same, the order of the values is imporant and differs between variants
	enum EFramePattern : uint8
	{
		None,
		Key,
		DeltaT2A,
		DeltaT1,
		DeltaT2B,
		DeltaT0,
	};

	static UE_API FString		 FramePatternNames[6];
	static constexpr int MaxNumSpatialLayers = 3;
	static constexpr int MaxNumTemporalLayers = 3;

	// Index of the buffer to store last frame for layer (`Sid`, `Tid`)
	int BufferIndex(int Sid, int Tid) const
	{
		return Tid * NumSpatialLayers + Sid;
	}

	bool DecodeTargetIsActive(int Sid, int Tid) const
	{
		return static_cast<bool>(ActiveDecodeTargets[Sid * NumTemporalLayers + Tid]);
	}

	void SetDecodeTargetIsActive(int Sid, int Tid, bool Value)
	{
		ActiveDecodeTargets[Sid * NumTemporalLayers + Tid] = Value;
	}

	UE_API EFramePattern				   NextPattern() const;
	UE_API bool						   TemporalLayerIsActive(int Tid) const;
	static UE_API EDecodeTargetIndication Dti(int Sid, int Tid, const FScalableVideoController::FLayerFrameConfig& Frame);

	const int			 NumSpatialLayers;
	const int			 NumTemporalLayers;
	const FIntFraction	 ResolutionFactor;

	EFramePattern LastPattern = EFramePattern::None;
	TBitArray<>	  CanReferenceT0FrameForSpatialId;
	TBitArray<>	  CanReferenceT1FrameForSpatialId;
	TArray<bool>  ActiveDecodeTargets;
};

// T1       0   0
//         /   /   / ...
// T0     0---0---0--
// Time-> 0 1 2 3 4
class FScalabilityStructureL1T2 : public FScalabilityStructureFullSvc
{
public:
	explicit FScalabilityStructureL1T2(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureFullSvc(1, 2, ResolutionFactor)
	{
	}
	~FScalabilityStructureL1T2() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

// T2       0   0   0   0
//          |  /    |  /
// T1       / 0     / 0  ...
//         |_/     |_/
// T0     0-------0------
// Time-> 0 1 2 3 4 5 6 7
class FScalabilityStructureL1T3 : public FScalabilityStructureFullSvc
{
public:
	explicit FScalabilityStructureL1T3(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureFullSvc(1, 3, ResolutionFactor)
	{
	}
	~FScalabilityStructureL1T3() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

// S1  0--0--0-
//     |  |  | ...
// S0  0--0--0-
class FScalabilityStructureL2T1 : public FScalabilityStructureFullSvc
{
public:
	explicit FScalabilityStructureL2T1(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureFullSvc(2, 1, ResolutionFactor)
	{
	}
	~FScalabilityStructureL2T1() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

// S1T1     0   0
//         /|  /|  /
// S1T0   0-+-0-+-0
//        | | | | | ...
// S0T1   | 0 | 0 |
//        |/  |/  |/
// S0T0   0---0---0--
// Time-> 0 1 2 3 4
class FScalabilityStructureL2T2 : public FScalabilityStructureFullSvc
{
public:
	explicit FScalabilityStructureL2T2(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureFullSvc(2, 2, ResolutionFactor)
	{
	}
	~FScalabilityStructureL2T2() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

// S1T2      4    ,8
// S1T1    / |  6' |
// S1T0   2--+-'+--+-...
//        |  |  |  |
// S0T2   |  3  | ,7
// S0T1   | /   5'
// S0T0   1----'-----...
// Time-> 0  1  2  3
class FScalabilityStructureL2T3 : public FScalabilityStructureFullSvc
{
public:
	explicit FScalabilityStructureL2T3(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureFullSvc(2, 3, ResolutionFactor)
	{
	}
	~FScalabilityStructureL2T3() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

// S2     0-0-0-
//        | | |
// S1     0-0-0-...
//        | | |
// S0     0-0-0-
// Time-> 0 1 2
class FScalabilityStructureL3T1 : public FScalabilityStructureFullSvc
{
public:
	explicit FScalabilityStructureL3T1(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureFullSvc(3, 1, ResolutionFactor)
	{
	}
	~FScalabilityStructureL3T1() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

// https://www.w3.org/TR/webrtc-svc/#L3T2*
class FScalabilityStructureL3T2 : public FScalabilityStructureFullSvc
{
public:
	explicit FScalabilityStructureL3T2(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureFullSvc(3, 2, ResolutionFactor)
	{
	}
	~FScalabilityStructureL3T2() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

// https://www.w3.org/TR/webrtc-svc/#L3T3*
class FScalabilityStructureL3T3 : public FScalabilityStructureFullSvc
{
public:
	explicit FScalabilityStructureL3T3(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureFullSvc(3, 3, ResolutionFactor)
	{
	}
	~FScalabilityStructureL3T3() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

#undef UE_API
