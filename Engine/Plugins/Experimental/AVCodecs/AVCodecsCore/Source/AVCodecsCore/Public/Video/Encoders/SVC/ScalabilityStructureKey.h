// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/GenericFrameInfo.h"
#include "Video/Encoders/SVC/ScalableVideoController.h"

#define UE_API AVCODECSCORE_API

class FScalabilityStructureKeySvc : public FScalableVideoController
{
public:
	UE_API FScalabilityStructureKeySvc(int NumSpatialLayers, int NumTemporalLayers);
	~FScalabilityStructureKeySvc() override = default;

	UE_API virtual FStreamLayersConfig StreamConfig() const override;

	UE_API virtual TArray<FScalableVideoController::FLayerFrameConfig> NextFrameConfig(bool bRestart) override;
	UE_API virtual FGenericFrameInfo									OnEncodeDone(const FScalableVideoController::FLayerFrameConfig& Config) override;
	UE_API virtual void												OnRatesUpdated(const FVideoBitrateAllocation& Bitrates) override;

private:
	// NOTE: This enum name is duplicated throughout the ScalabilityStructure variants
	// While it's name is the same, the order of the values is imporant and differs between variants
	enum EFramePattern : uint8
	{
		None,
		Key,
		DeltaT0,
		DeltaT2A,
		DeltaT1,
		DeltaT2B,
	};

	static constexpr int MaxNumSpatialLayers = 3;
	static constexpr int MaxNumTemporalLayers = 3;

	// Index of the buffer to store last frame for layer (`Sid`, `Tid`)
	int BufferIndex(int Sid, int Tid) const
	{
		return Tid * NumSpatialLayers + Sid;
	}

	bool DecodeTargetIsActive(int Sid, int Tid) const
	{
		return static_cast<bool>(ActiveDecodeTargets[Sid * NumTemporalLayers + Tid]);
	}

	void SetDecodeTargetIsActive(int Sid, int Tid, bool value)
	{
		ActiveDecodeTargets[Sid * NumTemporalLayers + Tid] = value;
	}

	UE_API bool						   TemporalLayerIsActive(int Tid) const;
	static UE_API EDecodeTargetIndication Dti(int Sid, int Tid, const FScalableVideoController::FLayerFrameConfig& config);

	UE_API TArray<FScalableVideoController::FLayerFrameConfig> KeyframeConfig();
	UE_API TArray<FScalableVideoController::FLayerFrameConfig> T0Config();
	UE_API TArray<FScalableVideoController::FLayerFrameConfig> T1Config();
	UE_API TArray<FScalableVideoController::FLayerFrameConfig> T2Config(EFramePattern Pattern);

	UE_API EFramePattern NextPattern(EFramePattern InLastPattern) const;

	const int NumSpatialLayers;
	const int NumTemporalLayers;

	EFramePattern LastPattern = EFramePattern::None;
	TBitArray<>	  SpatialIdIsEnabled;
	TBitArray<>	  CanReferenceT1FrameForSpatialId;
	TArray<bool>  ActiveDecodeTargets;
};

// S1  0--0--0-
//     |       ...
// S0  0--0--0-
class FScalabilityStructureL2T1Key : public FScalabilityStructureKeySvc
{
public:
	FScalabilityStructureL2T1Key()
		: FScalabilityStructureKeySvc(2, 1)
	{
	}
	~FScalabilityStructureL2T1Key() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

// S1T1     0   0
//         /   /   /
// S1T0   0---0---0
//        |         ...
// S0T1   | 0   0
//        |/   /   /
// S0T0   0---0---0
// Time-> 0 1 2 3 4
class FScalabilityStructureL2T2Key : public FScalabilityStructureKeySvc
{
public:
	FScalabilityStructureL2T2Key()
		: FScalabilityStructureKeySvc(2, 2)
	{
	}
	~FScalabilityStructureL2T2Key() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

class FScalabilityStructureL2T3Key : public FScalabilityStructureKeySvc
{
public:
	FScalabilityStructureL2T3Key()
		: FScalabilityStructureKeySvc(2, 3)
	{
	}
	~FScalabilityStructureL2T3Key() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

class FScalabilityStructureL3T1Key : public FScalabilityStructureKeySvc
{
public:
	FScalabilityStructureL3T1Key()
		: FScalabilityStructureKeySvc(3, 1)
	{
	}
	~FScalabilityStructureL3T1Key() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

class FScalabilityStructureL3T2Key : public FScalabilityStructureKeySvc
{
public:
	FScalabilityStructureL3T2Key()
		: FScalabilityStructureKeySvc(3, 2)
	{
	}
	~FScalabilityStructureL3T2Key() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

class FScalabilityStructureL3T3Key : public FScalabilityStructureKeySvc
{
public:
	FScalabilityStructureL3T3Key()
		: FScalabilityStructureKeySvc(3, 3)
	{
	}
	~FScalabilityStructureL3T3Key() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

#undef UE_API
