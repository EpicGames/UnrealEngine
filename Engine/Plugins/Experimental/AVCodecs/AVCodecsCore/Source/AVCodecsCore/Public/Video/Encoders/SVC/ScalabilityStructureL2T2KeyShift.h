// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/GenericFrameInfo.h"
#include "Video/Encoders/SVC/ScalableVideoController.h"

#define UE_API AVCODECSCORE_API

// S1T1     0   0
//         /   /   /
// S1T0   0---0---0
//        |        ...
// S0T1   |   0   0
//        |  /   /
// S0T0   0-0---0--
// Time-> 0 1 2 3 4
class FScalabilityStructureL2T2KeyShift : public FScalableVideoController
{
public:
	UE_API FScalabilityStructureL2T2KeyShift();
	~FScalabilityStructureL2T2KeyShift() override = default;

	UE_API virtual FStreamLayersConfig		  StreamConfig() const override;
	UE_API virtual FFrameDependencyStructure DependencyStructure() const override;

	UE_API virtual TArray<FScalableVideoController::FLayerFrameConfig> NextFrameConfig(bool bRestart) override;
	UE_API virtual FGenericFrameInfo									OnEncodeDone(const FScalableVideoController::FLayerFrameConfig& Config) override;
	UE_API virtual void												OnRatesUpdated(const FVideoBitrateAllocation& Bitrates) override;

private:
	// NOTE: This enum name is duplicated throughout the ScalabilityStructure variants
	// While it's name is the same, the order of the values is imporant and differs between variants
	enum EFramePattern : uint8
	{
		Key,
		Delta0,
		Delta1,
	};

	static constexpr int NumSpatialLayers = 2;
	static constexpr int NumTemporalLayers = 2;

	bool DecodeTargetIsActive(int Sid, int Tid) const
	{
		return static_cast<bool>(ActiveDecodeTargets[Sid * NumTemporalLayers + Tid]);
	}
	void SetDecodeTargetIsActive(int Sid, int Tid, bool value)
	{
		ActiveDecodeTargets[Sid * NumTemporalLayers + Tid] = value;
	}
	static UE_API EDecodeTargetIndication Dti(int Sid, int Tid, const FScalableVideoController::FLayerFrameConfig& Config);

	EFramePattern NextPattern = EFramePattern::Key;
	TArray<bool>  ActiveDecodeTargets;
};

#undef UE_API
