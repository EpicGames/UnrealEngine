// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/GenericFrameInfo.h"
#include "Video/Encoders/SVC/ScalableVideoController.h"

#define UE_API AVCODECSCORE_API

// Scalability structure with multiple independent spatial layers each with the
// same temporal layering.
class FScalabilityStructureSimulcast : public FScalableVideoController
{
public:
	UE_API FScalabilityStructureSimulcast(int NumSpatialLayers, int NumTemporalLayers, FIntFraction ResolutionFactor);
	virtual ~FScalabilityStructureSimulcast() override = default;

	UE_API virtual FStreamLayersConfig									StreamConfig() const override;
	UE_API virtual TArray<FScalableVideoController::FLayerFrameConfig> NextFrameConfig(bool bRestart) override;
	UE_API virtual FGenericFrameInfo									OnEncodeDone(const FScalableVideoController::FLayerFrameConfig& Config) override;
	UE_API virtual void												OnRatesUpdated(const FVideoBitrateAllocation& Bitrates) override;

private:
	// NOTE: This enum name is duplicated throughout the ScalabilityStructure variants
	// While it's name is the same, the order of the values is imporant and differs between variants
	enum EFramePattern : uint8
	{
		None,
		DeltaT2A,
		DeltaT1,
		DeltaT2B,
		DeltaT0,
	};
	static constexpr int MaxNumSpatialLayers = 3;
	static constexpr int MaxNumTemporalLayers = 3;

	// Index of the buffer to store last frame for layer (`Sid`, `Tid`)
	int BufferIndex(int Sid, int Tid) const
	{
		return Tid * NumSpatialLayers + Sid;
	}

	bool DecodeTargetIsActive(int Sid, int Tid) const
	{
		return static_cast<bool>(ActiveDecodeTargets[Sid * NumTemporalLayers + Tid]);
	}

	void SetDecodeTargetIsActive(int Sid, int Tid, bool Value)
	{
		ActiveDecodeTargets[Sid * NumTemporalLayers + Tid] = Value;
	}

	UE_API EFramePattern				   NextPattern() const;
	UE_API bool						   TemporalLayerIsActive(int Tid) const;
	static UE_API EDecodeTargetIndication Dti(int Sid, int Tid, const FScalableVideoController::FLayerFrameConfig& Config);

	const int	   NumSpatialLayers;
	const int	   NumTemporalLayers;
	FIntFraction   ResolutionFactor;

	EFramePattern LastPattern = EFramePattern::None;
	TBitArray<>	  CanReferenceT0FrameForSpatialId;
	TBitArray<>	  CanReferenceT1FrameForSpatialId;
	TArray<bool>  ActiveDecodeTargets;
};

// S1  0--0--0-
//             ...
// S0  0--0--0-
class FScalabilityStructureS2T1 : public FScalabilityStructureSimulcast
{
public:
	FScalabilityStructureS2T1(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureSimulcast(2, 1, ResolutionFactor)
	{
	}
	~FScalabilityStructureS2T1() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

class FScalabilityStructureS2T2 : public FScalabilityStructureSimulcast
{
public:
	explicit FScalabilityStructureS2T2(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureSimulcast(2, 2, ResolutionFactor) {}
	~FScalabilityStructureS2T2() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

// S1T2       3   7
//            |  /
// S1T1       / 5
//           |_/
// S1T0     1-------9...
//
// S0T2       2   6
//            |  /
// S0T1       / 4
//           |_/
// S0T0     0-------8...
// Time->   0 1 2 3 4
class FScalabilityStructureS2T3 : public FScalabilityStructureSimulcast
{
public:
	explicit FScalabilityStructureS2T3(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureSimulcast(2, 3, ResolutionFactor)
	{
	}
	~FScalabilityStructureS2T3() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

class FScalabilityStructureS3T1 : public FScalabilityStructureSimulcast
{
public:
	explicit FScalabilityStructureS3T1(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureSimulcast(3, 1, ResolutionFactor) {}
	~FScalabilityStructureS3T1() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

class FScalabilityStructureS3T2 : public FScalabilityStructureSimulcast
{
public:
	explicit FScalabilityStructureS3T2(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureSimulcast(3, 2, ResolutionFactor)
	{
	}
	~FScalabilityStructureS3T2() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

class FScalabilityStructureS3T3 : public FScalabilityStructureSimulcast
{
public:
	FScalabilityStructureS3T3(FIntFraction ResolutionFactor = { 1, 2 })
		: FScalabilityStructureSimulcast(3, 3, ResolutionFactor)
	{
	}
	~FScalabilityStructureS3T3() override = default;

	UE_API FFrameDependencyStructure DependencyStructure() const override;
};

#undef UE_API
