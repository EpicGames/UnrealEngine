// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/GenericFrameInfo.h"
#include "Video/Encoders/VideoBitrateAllocator.h"

#define UE_API AVCODECSCORE_API

// Controls how video should be encoded to be scalable. Outputs results as
// buffer usage configuration for encoder and enough details to communicate the
// scalability structure via dependency descriptor rtp header extension.
class FScalableVideoController
{
public:
	struct FIntFraction
	{
		int Num = 1;
		int Den = 2;
	};

	struct FStreamLayersConfig
	{
		int NumSpatialLayers = 1;
		int NumTemporalLayers = 1;
		// Indicates if frames can reference frames of a different resolution.
		bool bUsesReferenceScaling = true;
		// Spatial layers scaling. Frames with spatial_id = i expected to be encoded
		// with original_resolution * ScalingFactors[i].Num / ScalingFactors[i].Den
		FIntFraction ScalingFactors[4] = { { 1, 1 }, { 1, 1 }, { 1, 1 }, { 1, 1 } };
	};

	class FLayerFrameConfig
	{
	public:
		// Builders/setters.
		UE_API FLayerFrameConfig& Id(int Value);
		UE_API FLayerFrameConfig& Keyframe();
		UE_API FLayerFrameConfig& SpatialLayerId(int Value);
		UE_API FLayerFrameConfig& TemporalLayerId(int Value);
		UE_API FLayerFrameConfig& Reference(int BufferId);
		UE_API FLayerFrameConfig& Update(int BufferId);
		UE_API FLayerFrameConfig& ReferenceAndUpdate(int BufferId);

		// Getters.
		UE_API int								 GetId() const;
		UE_API bool							 GetIsKeyframe() const;
		UE_API int								 GetSpatialId() const;
		UE_API int								 GetTemporalId() const;
		UE_API const TArray<FCodecBufferUsage>& GetBuffers() const;

	private:
		// Id to match configuration returned by NextFrameConfig with
		// (possibly modified) configuration passed back via OnEncoderDone.
		// The meaning of the id is an implementation detail of
		// the FScalableVideoController.
		int LayerId = 0;

		// Indication frame should be encoded as a key frame. In particular when
		// `is_keyframe=true` property `FCodecBufferUsage::referenced` should be
		// ignored and treated as false.
		bool bIsKeyFrame = false;

		int SpatialId = 0;
		int TemporalId = 0;
		// Describes how encoder which buffers encoder allowed to reference and
		// which buffers encoder should update.
		TArray<FCodecBufferUsage> Buffers;
	};

	virtual ~FScalableVideoController() = default;

	// Returns video structure description for encoder to configure itself.
	virtual FStreamLayersConfig StreamConfig() const = 0;

	// Returns video structure description in format compatible with
	// dependency descriptor rtp header extension.
	virtual FFrameDependencyStructure DependencyStructure() const = 0;

	// Notifies Controller with updated bitrates per layer. In particular notifies
	// when certain layers should be disabled.
	// Controller shouldn't produce FLayerFrameConfig for disabled layers.
	virtual void OnRatesUpdated(const FVideoBitrateAllocation& Bitrates) = 0;

	// When `bRestart` is true, first `FLayerFrameConfig` should have `bIsKeyframe`
	// set to true.
	// Returned vector shouldn't be empty.
	virtual TArray<FLayerFrameConfig> NextFrameConfig(bool bRestart) = 0;

	// Returns configuration to pass to EncoderCallback.
	virtual FGenericFrameInfo OnEncodeDone(const FLayerFrameConfig& Config) = 0;
};

#undef UE_API
