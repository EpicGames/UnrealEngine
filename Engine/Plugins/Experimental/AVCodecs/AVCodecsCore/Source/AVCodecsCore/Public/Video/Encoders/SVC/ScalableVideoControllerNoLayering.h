// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/Encoders/SVC/ScalableVideoController.h"

#define UE_API AVCODECSCORE_API

class FScalableVideoControllerNoLayering : public FScalableVideoController
{
public:
	~FScalableVideoControllerNoLayering() override = default;

	UE_API virtual FStreamLayersConfig		  StreamConfig() const override;
	UE_API virtual FFrameDependencyStructure DependencyStructure() const override;

	UE_API virtual TArray<FScalableVideoController::FLayerFrameConfig> NextFrameConfig(bool bRestart) override;
	UE_API virtual FGenericFrameInfo									OnEncodeDone(const FScalableVideoController::FLayerFrameConfig& Config) override;
	UE_API virtual void												OnRatesUpdated(const FVideoBitrateAllocation& Bitrates) override;

private:
	bool bStart = true;
	bool bEnabled = true;
};

#undef UE_API
