// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/Encoders/VideoBitrateAllocator.h"
#include "Video/VideoEncoder.h"

#define UE_API AVCODECSCORE_API

class FVideoBitrateAllocatorSVC : public FVideoBitrateAllocator
{
public:
	// NOTE: Config stores its rates as kbps whereas this class will use bps. Convert to bps everywhere
	UE_API explicit FVideoBitrateAllocatorSVC(const FVideoEncoderConfig& Config);
	virtual ~FVideoBitrateAllocatorSVC() = default;

	UE_API FVideoBitrateAllocation Allocate(FVideoBitrateAllocationParameters Parameters) override;

	static UE_API uint32		  GetMaxBitrate(const FVideoEncoderConfig& Config);
	static UE_API uint32		  GetPaddingBitrate(const FVideoEncoderConfig& Config);
	static UE_API TArray<uint32> GetLayerStartBitrates(const FVideoEncoderConfig& Config);

private:
	struct FNumLayers
	{
		size_t Spatial = 1;
		size_t Temporal = 1;
	};

	static UE_API FNumLayers		GetNumLayers(const FVideoEncoderConfig& Config);
	UE_API FVideoBitrateAllocation GetAllocationNormalVideo(uint32 TotalBitrateBps, size_t FirstActiveLayer, size_t NumSpatialLayers) const;

	// Returns the number of layers that are active and have enough bitrate to
	// actually be enabled.
	UE_API size_t FindNumEnabledLayers(uint32 TargetRate) const;

	FVideoEncoderConfig	 Config;
	const FNumLayers	 NumLayers;
	const TArray<uint32> CumulativeLayerStartBitrates;
	size_t				 LastActiveLayerCount;
};

#undef UE_API
