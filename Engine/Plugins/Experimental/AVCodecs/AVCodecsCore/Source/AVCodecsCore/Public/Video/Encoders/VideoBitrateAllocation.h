// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/NumericLimits.h"
#include "Misc/Optional.h"
#include "Video/VideoEncoder.h"

#define UE_API AVCODECSCORE_API

class FVideoBitrateAllocation
{
public:
	static constexpr uint32 MaxBitrateBps = TNumericLimits<uint32>::Max();

	UE_API FVideoBitrateAllocation();

	virtual ~FVideoBitrateAllocation() = default;

	UE_API bool HasBitrate(uint64 SpatialIndex, uint64 TemporalIndex) const;

	UE_API uint32 GetBitrate(uint64 SpatialIndex, uint64 TemporalIndex) const;

	UE_API bool IsSpatialLayerUsed(uint64 SpatialIndex) const;

	UE_API uint32 GetSpatialLayerSumBitrate(uint64 SpatialIndex) const;

	UE_API uint32 GetTemporalLayerSumBitrate(uint64 SpatialIndex, uint64 TemporalIndex) const;

	UE_API TArray<uint32> GetTemporalLayerAllocation(uint64 SpatialIndex) const;

	UE_API uint32 GetSumBps() const;

	UE_API bool IsBwLimited() const;

	UE_API bool SetBitrate(size_t SpatialIndex, size_t TemporalIndex, uint32 BitrateBps);

	// Indicates if the allocation has some layers/streams disabled due to
	// low available bandwidth.
	UE_API void SetBwLimited(bool Limited);

	// Returns one FVideoBitrateAllocation for each spatial layer. This is used to
	// configure simulcast streams. Note that the length of the returned vector is
	// always 5, the optional is unset for unused layers.
	UE_API TArray<TOptional<FVideoBitrateAllocation>> GetSimulcastAllocations() const;

	inline bool operator==(const FVideoBitrateAllocation& Other) const
	{
		for (size_t si = 0; si < 5; ++si)
		{
			for (size_t ti = 0; ti < 4; ++ti)
			{
				if (Bitrates[si][ti] != Other.Bitrates[si][ti])
				{
					return false;
				}
			}
		}
		return true;
	}

	inline bool operator!=(const FVideoBitrateAllocation& Other) const
	{
		return !(*this == Other);
	}

private:
	uint32			  SumBps;
	TOptional<uint32> Bitrates[Video::MaxSpatialLayers][Video::MaxTemporalStreams];
	bool			  bIsBwLimited;
};

#undef UE_API
