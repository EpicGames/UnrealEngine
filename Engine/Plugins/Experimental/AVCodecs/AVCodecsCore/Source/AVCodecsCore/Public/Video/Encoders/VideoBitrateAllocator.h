// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "Video/Encoders/VideoBitrateAllocation.h"

#define UE_API AVCODECSCORE_API

class FVideoBitrateAllocationParameters
{
public:
	FVideoBitrateAllocationParameters(uint32 TotalBitrateBps, FFrameRate Framerate)
		: TotalBitrateBps(TotalBitrateBps)
		, StableBitrateBps(TotalBitrateBps)
		, Framerate(Framerate)
	{
	}
	FVideoBitrateAllocationParameters(uint32 TotalBitrateBps, uint32 StableBitrateBps, FFrameRate Framerate)
		: TotalBitrateBps(TotalBitrateBps)
		, StableBitrateBps(StableBitrateBps)
		, Framerate(Framerate)
	{
	}

	uint32 TotalBitrateBps;
	uint32 StableBitrateBps;
	FFrameRate Framerate;
};

class FVideoBitrateAllocator
{
public:
	FVideoBitrateAllocator() = default;
	~FVideoBitrateAllocator() = default;

	UE_API virtual FVideoBitrateAllocation GetAllocation(uint32 TotalBitrateBps, FFrameRate Framerate);

	UE_API virtual FVideoBitrateAllocation Allocate(FVideoBitrateAllocationParameters Parameters);
};

#undef UE_API
