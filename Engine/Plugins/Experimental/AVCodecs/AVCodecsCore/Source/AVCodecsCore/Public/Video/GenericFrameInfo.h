// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Video/DependencyDescriptor.h"

#define UE_API AVCODECSCORE_API

struct FCodecBufferUsage
{
	int32 Id = 0;
	bool  bReferenced = false;
	bool  bUpdated = false;
};

class FGenericFrameInfo : public FFrameDependencyTemplate
{
public:
	UE_API FGenericFrameInfo();

	friend bool operator==(const FGenericFrameInfo& Lhs, const FGenericFrameInfo& Rhs)
	{
		return Lhs.SpatialId == Rhs.SpatialId
			&& Lhs.TemporalId == Rhs.TemporalId
			&& Lhs.DecodeTargetIndications == Rhs.DecodeTargetIndications
			&& Lhs.FrameDiffs == Rhs.FrameDiffs
			&& Lhs.ChainDiffs == Rhs.ChainDiffs;
	}

	TArray<FCodecBufferUsage> EncoderBuffers;
	TArray<bool>			  PartOfChain;
	TArray<bool>			  ActiveDecodeTargets;
};

#undef UE_API
