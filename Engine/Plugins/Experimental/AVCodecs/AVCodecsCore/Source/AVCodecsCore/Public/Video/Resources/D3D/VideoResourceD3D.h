// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS

#include "Templates/RefCounting.h"

#include "AVContext.h"
#include "Video/VideoResource.h"

#pragma warning(push)
#pragma warning(disable: 4005)

THIRD_PARTY_INCLUDES_START
#include "Windows/AllowWindowsPlatformTypes.h"
#include <d3d11.h>
#include <mfobjects.h>
#include <mftransform.h>
#include <mfapi.h>
#include <mferror.h>
#include <mfidl.h>
#include <codecapi.h>
#include <shlwapi.h>
#include <mfreadwrite.h>
#include <d3d11_1.h>
#include <d3d12.h>
#include <dxgi1_4.h>
#include "Windows/HideWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

/**
 * D3D11 and D3D12 platform video context and resource.
 */

class FVideoContextD3D11 : public FAVContext
{
public:
	TRefCountPtr<ID3D11Device> Device;

	AVCODECSCORE_API FVideoContextD3D11(TRefCountPtr<ID3D11Device> const& Device);
};

class FVideoContextD3D12 : public FAVContext
{
public:
	TRefCountPtr<ID3D12Device> Device;

	AVCODECSCORE_API FVideoContextD3D12(TRefCountPtr<ID3D12Device> const& Device);
};

class FVideoResourceD3D11 : public TVideoResource<FVideoContextD3D11>
{
private:
	TRefCountPtr<ID3D11Texture2D> Raw;
	HANDLE SharedHandle = nullptr;

public:
	static AVCODECSCORE_API FVideoDescriptor GetDescriptorFrom(TSharedRef<FAVDevice> const& Device, TRefCountPtr<ID3D11Texture2D> const& Raw);

	inline TRefCountPtr<ID3D11Texture2D> const& GetRaw() const { return Raw; }
	inline HANDLE const& GetSharedHandle() const { return SharedHandle; }

	AVCODECSCORE_API FVideoResourceD3D11(TSharedRef<FAVDevice> const& Device, TRefCountPtr<ID3D11Texture2D> const& Raw, FAVLayout const& Layout);

	AVCODECSCORE_API virtual FAVResult Validate() const override;
};

class FVideoResourceD3D12 : public TVideoResource<FVideoContextD3D12>
{
public:
	struct FRawD3D12
	{
		TRefCountPtr<ID3D12Resource> D3DResource;
		HANDLE D3DResourceShared = nullptr;
		TRefCountPtr<ID3D12Heap> D3DHeap;
		HANDLE D3DHeapShared = nullptr;
		TRefCountPtr<ID3D12Fence> D3DFence;
		HANDLE D3DFenceShared = nullptr;
		uint64 FenceValue;
	};

private:
	FRawD3D12 Raw;

public:
	static AVCODECSCORE_API FVideoDescriptor GetDescriptorFrom(TSharedRef<FAVDevice> const& Device, FRawD3D12 const& Raw);

	inline TRefCountPtr<ID3D12Resource> const& GetResource() const { return Raw.D3DResource; }
	inline HANDLE const& GetResourceSharedHandle() const { return Raw.D3DResourceShared; }
	
	inline TRefCountPtr<ID3D12Heap> const& GetHeap() const { return Raw.D3DHeap; }
	inline HANDLE const& GetHeapSharedHandle() const { return Raw.D3DHeapShared; }

	inline TRefCountPtr<ID3D12Fence> const& GetFence() const { return Raw.D3DFence; };
	inline HANDLE const& GetFenceSharedHandle() const { return Raw.D3DFenceShared; }
	inline uint64 const& GetFenceValue() const { return Raw.FenceValue; }

	AVCODECSCORE_API uint64 GetSizeInBytes();

	AVCODECSCORE_API FVideoResourceD3D12(TSharedRef<FAVDevice> const& Device, FRawD3D12 const& Raw, FAVLayout const& Layout);
	AVCODECSCORE_API FVideoResourceD3D12(TSharedRef<FAVDevice> const& Device, FRawD3D12 const& Raw, FAVLayout const& Layout, FVideoDescriptor const& Descriptor);
	AVCODECSCORE_API virtual ~FVideoResourceD3D12() override;

	AVCODECSCORE_API virtual FAVResult Validate() const override;
};

DECLARE_TYPEID(FVideoContextD3D11, AVCODECSCORE_API);
DECLARE_TYPEID(FVideoContextD3D12, AVCODECSCORE_API);
DECLARE_TYPEID(FVideoResourceD3D11, AVCODECSCORE_API);
DECLARE_TYPEID(FVideoResourceD3D12, AVCODECSCORE_API);

#endif
