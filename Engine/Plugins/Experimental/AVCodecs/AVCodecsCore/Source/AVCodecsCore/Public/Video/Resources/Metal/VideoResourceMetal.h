// Copyright Epic Games, Inc. All Rights Reserved.

#if AVCODECS_USE_METAL

#pragma once

#include "Templates/RefCounting.h"

#include "AVContext.h"
#include "Video/VideoResource.h"
#include "Containers/ResourceArray.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

THIRD_PARTY_INCLUDES_START
#include <CoreVideo/CoreVideo.h>
#include "MetalInclude.h"
THIRD_PARTY_INCLUDES_END


/**
 * Metal platform video context and resource.
 */

class FVideoContextMetal : public FAVContext
{
public:
	MTL::Device* Device;

	AVCODECSCORE_API FVideoContextMetal(MTL::Device* Device);
};


class FVideoResourceMetal : public TVideoResource<FVideoContextMetal>
{
private:
    MTL::Texture* Raw;
	MTL::Buffer* StagingBuffer;

public:
	static AVCODECSCORE_API FVideoDescriptor GetDescriptorFrom(TSharedRef<FAVDevice> const& Device, MTL::Texture* Raw);

	inline MTL::Texture* GetRaw() const { return Raw; }

	AVCODECSCORE_API FVideoResourceMetal(TSharedRef<FAVDevice> const& Device, MTL::Texture* Raw, FAVLayout const& Layout);
    AVCODECSCORE_API virtual ~FVideoResourceMetal() override;
	
	AVCODECSCORE_API FAVResult CopyFrom(CVPixelBufferRef Other);
    
	AVCODECSCORE_API virtual FAVResult Validate() const override;

	// Non-copyable and non-movable to prevent double-release of Metal resources.
	FVideoResourceMetal(FVideoResourceMetal const&) = delete;
	FVideoResourceMetal(FVideoResourceMetal&&) = delete;
	FVideoResourceMetal& operator=(FVideoResourceMetal const&) = delete;
	FVideoResourceMetal& operator=(FVideoResourceMetal&&) = delete;
};

DECLARE_TYPEID(FVideoContextMetal, AVCODECSCORE_API);
DECLARE_TYPEID(FVideoResourceMetal, AVCODECSCORE_API);

#endif
