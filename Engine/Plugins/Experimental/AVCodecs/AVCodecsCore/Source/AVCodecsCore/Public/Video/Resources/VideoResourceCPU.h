// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVContext.h"
#include "Video/VideoResource.h"

#define UE_API AVCODECSCORE_API

/**
 * CPU video context and resource.
 */

class FVideoContextCPU : public FAVContext
{
public:
	UE_API FVideoContextCPU();
};

class FVideoResourceCPU : public TVideoResource<FVideoContextCPU>
{
private:
	TSharedPtr<uint8> Raw;

public:
	inline TSharedPtr<uint8> const& GetRaw() const { return Raw; }
	inline void					 SetRaw(TSharedPtr<uint8> RawData) { Raw = RawData; }

	UE_API FVideoResourceCPU(TSharedRef<FAVDevice> const& Device, TSharedPtr<uint8> const& Raw, FAVLayout const& Layout, FVideoDescriptor const& Descriptor);
	virtual ~FVideoResourceCPU() override = default;

	UE_API virtual FAVResult Validate() const override;
};

class FResolvableVideoResourceCPU : public TResolvableVideoResource<FVideoResourceCPU>
{
protected:
	UE_API virtual TSharedPtr<FVideoResourceCPU> TryResolve(TSharedPtr<FAVDevice> const& Device, FVideoDescriptor const& Descriptor) override;

private:
	UE_API uint32 GetStrideInterleavedOrLuma(FVideoDescriptor const& Descriptor);
	UE_API uint32 GetSize(FVideoDescriptor const& Descriptor);
};

DECLARE_TYPEID(FVideoContextCPU, AVCODECSCORE_API);
DECLARE_TYPEID(FVideoResourceCPU, AVCODECSCORE_API);
DECLARE_TYPEID(FResolvableVideoResourceCPU, AVCODECSCORE_API);

#undef UE_API
