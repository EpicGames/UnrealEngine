// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if AVCODECS_USE_VULKAN

#include "AVContext.h"
#include "Video/VideoResource.h"

THIRD_PARTY_INCLUDES_START
#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#include "vulkan_core.h"

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif
THIRD_PARTY_INCLUDES_END

/**
 * Vulkan platform video context and resource.
 */

class FVideoContextVulkan : public FAVContext
{
public:
	VkInstance Instance;
	VkDevice Device;
	VkPhysicalDevice PhysicalDevice;
	TFunction<PFN_vkVoidFunction(const char*)> vkGetDeviceProcAddr;

	AVCODECSCORE_API FVideoContextVulkan(VkInstance const& Instance, VkDevice const& Device, VkPhysicalDevice const& PhysicalDevice, TFunction<PFN_vkVoidFunction(const char*)> const& GetDeviceProcAddrFunc);
};

#if PLATFORM_WINDOWS
typedef HANDLE FVulkanSharedHandle;
#else
typedef int FVulkanSharedHandle;
#endif

class FVideoResourceVulkan : public TVideoResource<FVideoContextVulkan>
{
private:
	VkDeviceMemory DeviceMemory;
	FVulkanSharedHandle DeviceMemorySharedHandle = FVulkanSharedHandle();
	VkFence Fence;
	FVulkanSharedHandle FenceSharedHandle = FVulkanSharedHandle();

public:
	inline VkDeviceMemory GetRaw() const { return DeviceMemory; }
	inline FVulkanSharedHandle const& GetSharedHandle() const { return DeviceMemorySharedHandle; }
	inline FVulkanSharedHandle const& GetFenceSharedHandle() const { return FenceSharedHandle; }

	AVCODECSCORE_API FVideoResourceVulkan(TSharedRef<FAVDevice> const& Device, VkDeviceMemory Raw, FAVLayout const& Layout, FVideoDescriptor const& Descriptor);
	virtual ~FVideoResourceVulkan() override = default;

	AVCODECSCORE_API virtual FAVResult Validate() const override;
};

DECLARE_TYPEID(FVideoContextVulkan, AVCODECSCORE_API);
DECLARE_TYPEID(FVideoResourceVulkan, AVCODECSCORE_API);

#endif
