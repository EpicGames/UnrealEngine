// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/CodecUtils/CodecUtilsH264.h"
#include "Video/CodecUtils/CodecUtilsVP9.h"
#include "Video/DependencyDescriptor.h"
#include "Video/GenericFrameInfo.h"
#include "Video/VideoConfig.h"

struct FCodecSpecificInfoVP8
{
	bool  bNonReference;
	uint8 TemporalIdx;
	bool  bLayerSync;
	int8  KeyIdx; // Negative value to skip keyIdx.

	// Used to generate the list of dependency frames.
	// `referencedBuffers` and `updatedBuffers` contain buffer IDs.
	// Note that the buffer IDs here have a one-to-one mapping with the actual
	// codec buffers, but the exact mapping (i.e. whether 0 refers to Last,
	// to Golden or to Arf) is not pre-determined.
	// More references may be specified than are strictly necessary, but not less.
	// TODO(bugs.webrtc.org/10242): Remove `useExplicitDependencies` once all
	// encoder-wrappers are updated.
	bool					bUseExplicitDependencies;
	static constexpr size_t BuffersCount = 3;
	size_t					ReferencedBuffers[BuffersCount];
	size_t					ReferencedBuffersCount;
	size_t					UpdatedBuffers[BuffersCount];
	size_t					UpdatedBuffersCount;
};

struct FCodecSpecificInfoVP9
{

	bool bFirstFrameInPicture; // First frame, increment picture_id.
	bool bInterPicPredicted;   // This layer frame is dependent on previously
							   // coded frame(s).
	bool bFlexibleMode;
	bool bSSDataAvailable;
	bool bNonRefForInterLayerPred;

	uint8 TemporalIdx;
	bool  bTemporalUpSwitch;
	bool  bInterLayerPredicted; // Frame is dependent on directly lower spatial
								// layer frame.
	uint8 GofIdx;

	// SS data.
	size_t		NumSpatialLayers; // Always populated.
	size_t		FirstActiveLayer;
	bool		bSpatialLayerResolutionPresent;
	uint16		Width[UE::AVCodecCore::VP9::MaxNumberOfSpatialLayers];
	uint16		Height[UE::AVCodecCore::VP9::MaxNumberOfSpatialLayers];
	UE::AVCodecCore::VP9::FGroupOfFramesInfo Gof;

	// Frame reference data.
	uint8 NumRefPics;
	uint8 PDiff[UE::AVCodecCore::VP9::MaxRefPics];
};

struct FCodecSpecificInfoH264
{
	EH264PacketizationMode PacketizationMode;
	uint8				   TemporalIdx;
	bool				   bBaseLayerSync;
	bool				   bIdrFrame;
};

union AVCODECSCORE_API FCodecSpecificInfoUnion
{
	FCodecSpecificInfoVP8  VP8;
	FCodecSpecificInfoVP9  VP9;
	FCodecSpecificInfoH264 H264;
};

struct FCodecSpecificInfo
{
	FCodecSpecificInfo()
		: Codec(EVideoCodec::Undefined)
	{
		memset(&CodecSpecific, 0, sizeof(CodecSpecific));
	}
	FCodecSpecificInfo(const FCodecSpecificInfo&) = default;
	~FCodecSpecificInfo() = default;

	EVideoCodec							 Codec;
	FCodecSpecificInfoUnion				 CodecSpecific;
	bool								 bEndOfPicture = true;
	TOptional<FGenericFrameInfo>		 GenericFrameInfo;
	TOptional<FFrameDependencyStructure> TemplateStructure;
};
