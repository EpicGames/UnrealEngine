// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVCoder.h"

#include "VideoConfig.generated.h"

UENUM()
enum class EVideoCodec : uint8
{
	Undefined,
	H264,
	H265,
	VP8,
	VP9,
	AV1,
	MAX
};

struct FVideoConfig : public FAVConfig
{
	FVideoConfig(EAVPreset Preset = EAVPreset::Default)
		: FAVConfig(Preset)
		, Codec(EVideoCodec::Undefined)
	{
	}

	EVideoCodec Codec;
};