// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVCoder.h"
#include "Video/VideoConfig.h"
#include "Video/VideoPacket.h"
#include "Video/VideoResource.h"

/*
 * Implementation of Video Decoding domain, see TAVCoder for inheritance model
 */

struct FVideoDecoderConfig : public FVideoConfig
{
public:
	FVideoDecoderConfig(EAVPreset Preset = EAVPreset::Default)
		: FVideoConfig(Preset) {}
};

/**
 * Video decoder with a factory, that supports typesafe resource handling and configuration.
 *
 * @see TAVCodec
 */
template <typename TResource = void, typename TConfig = void>
class TVideoDecoder : public TAVCoder<TVideoDecoder, TResource, TConfig>
{
public:
	/**
	 * Wrapper decoder that transforms resource/config types for use with a differently typed child decoder.
	 *
	 * @tparam TChildResource Type of child resource.
	 * @tparam TChildConfig Type of child config.
	 */
	template <typename TChildResource, typename TChildConfig>
	class TWrapper : public TAVCoder<TVideoDecoder, TResource, TConfig>::template TWrapper<TChildResource, TChildConfig>
	{
	public:
		TWrapper(TSharedRef<TVideoDecoder<TChildResource, TChildConfig>> const& Child)
			: TAVCoder<TVideoDecoder, TResource, TConfig>::template TWrapper<TChildResource, TChildConfig>(Child)
		{
		}

		virtual FAVResult SendPacket(FVideoPacket const& Packet) override
		{
			if (!this->IsOpen())
			{
				return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"));
			}

			FAVResult Result = this->ApplyConfig();
			if (Result.IsNotSuccess())
			{
				return Result;
			}

			return this->Child->SendPacket(Packet);
		}

		virtual FAVResult ReceiveFrame(TResolvableVideoResource<TResource>& InOutResource) override
		{
			if (!this->IsOpen())
			{
				return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"));
			}

			TDelegatedVideoResource<TChildResource> WrappedResource([&InOutResource](TSharedPtr<TChildResource>& InOutResult, TSharedPtr<FAVDevice> const& InDevice, FVideoDescriptor const& Descriptor) {
				InOutResult = nullptr;

				if (InOutResource.Resolve(InDevice, Descriptor))
				{
					InOutResult = InOutResource->template PinMapping<TChildResource>();
					if (!InOutResult.IsValid() || InOutResult->Validate().IsNotSuccess())
					{
						if (FAVExtension::TransformResource<TChildResource, TResource>(InOutResult, InOutResource).IsNotSuccess())
						{
							InOutResult.Reset();
						}
					}
				}
			});

			return this->Child->ReceiveFrame(WrappedResource);
		}
	};

	/**
	 * Get generic configuration values.
	 *
	 * @return The current minimal configuration.
	 */
	virtual FVideoDecoderConfig GetMinimalConfig() override
	{
		FVideoDecoderConfig MinimalConfig;
		FAVExtension::TransformConfig<FVideoDecoderConfig, TConfig>(MinimalConfig, this->GetPendingConfig());

		return MinimalConfig;
	}

	/**
	 * Set generic configuration values.
	 *
	 * @param MinimalConfig New minimal configuration to set.
	 */
	virtual void SetMinimalConfig(FVideoDecoderConfig const& MinimalConfig) override
	{
		FAVExtension::TransformConfig<TConfig, FVideoDecoderConfig>(this->EditPendingConfig(), MinimalConfig);
	}
};

/**
 * Video decoder with a factory, that supports typesafe resource handling.
 *
 * @see TAVCodec
 */
template <typename TResource>
class TVideoDecoder<TResource> : public TAVCoder<TVideoDecoder, TResource>
{
public:
	/**
	 * Read a finished frame out of the codec.
	 *
	 * @param InOutResource Deferred resource that can be resolved by the decoder in a desired format/size/etc. Passed by reference inout. @see TResolvableVideoResource
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult ReceiveFrame(TResolvableVideoResource<TResource>& InOutResource) = 0;
};

/**
 * Video decoder with a factory.
 *
 * @see TAVCodec
 */
template <>
class TVideoDecoder<> : public TAVCoder<TVideoDecoder>
{
public:
	/**
	 * Send a packet to the underlying codec architecture.
	 *
	 * @param Packet Packet holding the frame data. An empty packet will perform a flush (@see FlushPackets) and invalidate the underlying architecture.
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult SendPacket(FVideoPacket const& Packet) = 0;

	/**
	 * Flush remaining frames and invalidate the underlying architecture.
	 *
	 * @return Result of the operation, @see FAVResult.
	 */
	FAVResult FlushFrames()
	{
		return SendPacket(FVideoPacket());
	}

	/**
	 * Get generic configuration values.
	 *
	 * @return The current minimal configuration.
	 */
	virtual FVideoDecoderConfig GetMinimalConfig() = 0;

	/**
	 * Set generic configuration values.
	 *
	 * @param MinimalConfig New minimal configuration to set.
	 */
	virtual void SetMinimalConfig(FVideoDecoderConfig const& MinimalConfig) = 0;
};

typedef TVideoDecoder<> FVideoDecoder;

DECLARE_TYPEID(FVideoDecoder, AVCODECSCORE_API);