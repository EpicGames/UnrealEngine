// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVCoder.h"
#include "AVConstants.h"
#include "Video/VideoConfig.h"
#include "Video/VideoPacket.h"
#include "Video/VideoResource.h"

#include "VideoEncoder.generated.h"

/*
 * Implementation of Video Encoding domain, see TAVCoder for inheritance model
 */
UENUM()
enum class EScalabilityMode : uint8
{
	L1T1 = 0,
	L1T2,
	L1T3,
	L2T1,
	L2T1h,
	L2T1_KEY,
	L2T2,
	L2T2h,
	L2T2_KEY,
	L2T2_KEY_SHIFT,
	L2T3,
	L2T3h,
	L2T3_KEY,
	L3T1,
	L3T1h,
	L3T1_KEY,
	L3T2,
	L3T2h,
	L3T2_KEY,
	L3T3,
	L3T3h,
	L3T3_KEY,
	S2T1,
	S2T1h,
	S2T2,
	S2T2h,
	S2T3,
	S2T3h,
	S3T1,
	S3T1h,
	S3T2,
	S3T2h,
	S3T3,
	S3T3h,
	None
};

AVCODECSCORE_API extern TMap<FString, EScalabilityMode> const ScalabilityModeMap;

inline TOptional<EScalabilityMode> ScalabilityModeFromString(const FString& ModeString)
{
	if (ScalabilityModeMap.Find(ModeString) != nullptr)
	{
		return ScalabilityModeMap[ModeString];
	}

	return FNullOpt(0);
}

UENUM()
enum class ERateControlMode : uint8
{
	Unknown,
	ConstQP,
	VBR,
	CBR
};

UENUM()
enum class EMultipassMode : uint8
{
	Unknown,
	Disabled,
	Quarter,
	Full
};

// TODO (william.belcher): Use reasonable defaults set elsewhere
struct FSpatialLayer
{
	uint32 Width = 1920;
	uint32 Height = 1080;
	uint32 Framerate = 60;
	uint8  NumberOfTemporalLayers = 1;
	int32  MaxBitrate = 20000000;
	int32  TargetBitrate = 10000000;
	int32  MinBitrate = 5000000;
	int32  MaxQP = 0;
	bool   bActive = false;

	friend bool operator==(const FSpatialLayer& Lhs, const FSpatialLayer& Rhs)
	{
		return Lhs.Width == Rhs.Width
			&& Lhs.Height == Rhs.Height
			&& Lhs.Framerate == Rhs.Framerate
			&& Lhs.NumberOfTemporalLayers == Rhs.NumberOfTemporalLayers
			&& Lhs.MaxBitrate == Rhs.MaxBitrate
			&& Lhs.TargetBitrate == Rhs.TargetBitrate
			&& Lhs.MinBitrate == Rhs.MinBitrate
			&& Lhs.MaxQP == Rhs.MaxQP
			&& Lhs.bActive == Rhs.bActive;
	}
};

// TODO (william.belcher): Use reasonable defaults set elsewhere
struct FVideoEncoderConfig : public FVideoConfig
{
public:
	uint32 Width = 1920;
	uint32 Height = 1080;

	uint32 TargetFramerate = 60;

	// Individual coder implementations check for -1 bitrate and assign value
	int32 TargetBitrate = -1;
	int32 MaxBitrate = -1;
	int32 MinBitrate = -1;
	// Advanced bitrate settings. Used for situations such as simulcast / SVC
	TOptional<int32> Bitrates[Video::MaxSpatialLayers][Video::MaxTemporalStreams];

	int32 MinQuality = -1;
	int32 MaxQuality = -1;

	ERateControlMode RateControlMode = ERateControlMode::CBR;
	uint8			 bFillData : 1;

	EScalabilityMode ScalabilityMode = EScalabilityMode::None;

	uint8		  NumberOfSpatialLayers = 1;
	uint8		  NumberOfTemporalLayers = 1;
	FSpatialLayer SpatialLayers[Video::MaxSpatialLayers];

	uint8		  NumberOfSimulcastStreams;
	FSpatialLayer SimulcastStreams[Video::MaxSimulcastStreams];

	// TODO (Remove and derive from latency mode)
	uint32		   KeyframeInterval = 0;
	EMultipassMode MultipassMode = EMultipassMode::Full;

	FVideoEncoderConfig(EAVPreset Preset = EAVPreset::Default)
		: FVideoConfig(Preset)
		, bFillData(false)
	{
		switch (Preset)
		{
			case EAVPreset::UltraLowQuality:
				TargetBitrate = 500000;
				MaxBitrate = 1000000;

				MinQuality = 0;
				MaxQuality = 33;

				RateControlMode = ERateControlMode::CBR;

				break;
			case EAVPreset::LowQuality:
				TargetBitrate = 3000000;
				MaxBitrate = 4500000;

				MinQuality = 0;
				MaxQuality = 50;

				RateControlMode = ERateControlMode::CBR;

				break;
			case EAVPreset::Default:
				TargetBitrate = 5000000;
				MaxBitrate = 12500000;

				MinQuality = 25;
				MaxQuality = 75;

				RateControlMode = ERateControlMode::CBR;

				break;
			case EAVPreset::HighQuality:
				TargetBitrate = 10000000;
				MaxBitrate = 20000000;

				MinQuality = 50;
				MaxQuality = 100;

				RateControlMode = ERateControlMode::VBR;

				break;
			case EAVPreset::Lossless:
				TargetBitrate = 0;
				MaxBitrate = 0;

				MinQuality = -1;
				MaxQuality = -1;

				RateControlMode = ERateControlMode::ConstQP;

				break;
		}
	}
};

/**
 * Video encoder with a factory, that supports typesafe resource handling and configuration.
 *
 * @see TAVCodec
 */
template <typename TResource = void, typename TConfig = void>
class TVideoEncoder : public TAVCoder<TVideoEncoder, TResource, TConfig>
{
public:
	/**
	 * Wrapper encoder that transforms resource/config types for use with a differently typed child encoder.
	 *
	 * @tparam TChildResource Type of child resource.
	 * @tparam TChildConfig Type of child config.
	 */
	template <typename TChildResource, typename TChildConfig>
	class TWrapper : public TAVCoder<TVideoEncoder, TResource, TConfig>::template TWrapper<TChildResource, TChildConfig>
	{
	public:
		TWrapper(TSharedRef<TVideoEncoder<TChildResource, TChildConfig>> const& Child)
			: TAVCoder<TVideoEncoder, TResource, TConfig>::template TWrapper<TChildResource, TChildConfig>(Child)
		{
		}

		virtual FAVResult SendFrame(TSharedPtr<TResource> const& Resource, uint32 Timestamp, bool bForceKeyframe = false) override
		{
			if (!this->IsOpen())
			{
				return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"));
			}

			FAVResult Result = this->ApplyConfig();
			if (Result.IsNotSuccess())
			{
				return Result;
			}

			if (!Resource.IsValid())
			{
				return this->Child->SendFrame(nullptr, Timestamp, bForceKeyframe);
			}

			FScopeLock const Lock = Resource->LockScope();

			TSharedPtr<TChildResource> MappedResource = Resource->template PinMapping<TChildResource>();
			if (!MappedResource.IsValid() || MappedResource->Validate().IsNotSuccess())
			{
				Result = FAVExtension::TransformResource<TChildResource, TResource>(MappedResource, Resource);
				if (Result.IsNotSuccess())
				{
					MappedResource.Reset();

					return Result;
				}
			}

			return this->Child->SendFrame(MappedResource, Timestamp, bForceKeyframe);
		}

		virtual FAVResult ReceivePacket(FVideoPacket& OutPacket) override
		{
			if (!this->IsOpen())
			{
				return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"));
			}

			return this->Child->ReceivePacket(OutPacket);
		}
	};

	/**
	 * Get generic configuration values.
	 *
	 * @return The current minimal configuration.
	 */
	virtual FVideoEncoderConfig GetMinimalConfig() override
	{
		FVideoEncoderConfig MinimalConfig;
		FAVExtension::TransformConfig<FVideoEncoderConfig, TConfig>(MinimalConfig, this->GetPendingConfig());

		return MinimalConfig;
	}

	/**
	 * Set generic configuration values.
	 *
	 * @param MinimalConfig New minimal configuration to set.
	 */
	virtual void SetMinimalConfig(FVideoEncoderConfig const& MinimalConfig) override
	{
		FAVExtension::TransformConfig<TConfig, FVideoEncoderConfig>(this->EditPendingConfig(), MinimalConfig);
	}
};

/**
 * Video encoder with a factory, that supports typesafe resource handling.
 *
 * @see TAVCodec
 */
template <typename TResource>
class TVideoEncoder<TResource> : public TAVCoder<TVideoEncoder, TResource>
{
public:
	/**
	 * Send a frame to the underlying codec architecture.
	 *
	 * @param Resource Resource holding the frame data. An invalid resource will perform a flush (@see FlushPackets) and invalidate the underlying architecture.
	 * @param Timestamp Recorded timestamp of the frame.
	 * @param bForceKeyframe Whether the frame should be forced to be a keyframe.
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult SendFrame(TSharedPtr<TResource> const& Resource, uint32 Timestamp, bool bForceKeyframe = false) = 0;

	/**
	 * Flush remaining packets and invalidate the underlying architecture.
	 *
	 * @return Result of the operation, @see FAVResult.
	 */
	FAVResult FlushPackets()
	{
		return SendFrame(nullptr, 0);
	}

	/**
	 * Flush remaining packets and invalidate the underlying architecture.
	 *
	 * @return Result of the operation, @see FAVResult.
	 */
	FAVResult FlushAndReceivePackets(TArray<FVideoPacket>& OutPackets)
	{
		FAVResult Result = FlushPackets();
		if (Result.IsNotSuccess())
		{
			return Result;
		}

		return this->ReceivePackets(OutPackets);
	}
};

/**
 * Video encoder with a factory.
 *
 * @see TAVCodec
 */
template <>
class TVideoEncoder<> : public TAVCoder<TVideoEncoder>
{
public:
	/**
	 * Read a finished packet out of the codec.
	 *
	 * @param OutPacket Output packet if one is complete.
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult ReceivePacket(FVideoPacket& OutPacket) = 0;

	/**
	 * Read all finished packets out of the codec.
	 *
	 * @param OutPackets Output array of completed packets.
	 * @return Result of the operation, @see FAVResult.
	 */
	virtual FAVResult ReceivePackets(TArray<FVideoPacket>& OutPackets)
	{
		FAVResult Result;

		FVideoPacket Packet;
		while ((Result = ReceivePacket(Packet)).IsSuccess())
		{
			OutPackets.Add(Packet);
		}

		return Result;
	}

	/**
	 * Get generic configuration values.
	 *
	 * @return The current minimal configuration.
	 */
	virtual FVideoEncoderConfig GetMinimalConfig() = 0;

	/**
	 * Set generic configuration values.
	 *
	 * @param MinimalConfig New minimal configuration to set.
	 */
	virtual void SetMinimalConfig(FVideoEncoderConfig const& MinimalConfig) = 0;
};

typedef TVideoEncoder<> FVideoEncoder;

DECLARE_TYPEID(FVideoEncoder, AVCODECSCORE_API);
