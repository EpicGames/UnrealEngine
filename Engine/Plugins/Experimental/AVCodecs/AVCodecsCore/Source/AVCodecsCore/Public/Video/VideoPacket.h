// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVPacket.h"
#include "Templates/SharedPointerFwd.h"
#include "Video/VideoCodecSpecificInfo.h"

/**
 * Video-specific packet container.
 */
struct FVideoPacket : public FAVPacket
{
public:
	/**
	 * QP that the raw data was encoded with.
	 */
	uint32 QP;

	/**
	 * Whether this frame is a keyframe.
	 */
	uint8 bIsKeyframe : 1;

	/**
	 * Struct holding advanced information on the video packet
	 */
	FCodecSpecificInfo CodecSpecificInfo;

	/**
	 * Spatial index of this packet
	 */
	TOptional<int> SpatialIndex;

	/**
	 * Temporal index of this packet
	 */
	TOptional<int> TemporalIndex;

	FVideoPacket() = default;
	FVideoPacket(TSharedPtr<uint8> const& DataPtr, uint64 DataSize, uint64 Timestamp, uint64 Index, uint32 QP, bool bIsKeyframe)
		: FAVPacket(DataPtr, DataSize, Timestamp, Index)
		, QP(QP)
		, bIsKeyframe(bIsKeyframe)
	{
	}
};
