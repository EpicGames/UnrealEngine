// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVLayout.h"
#include "AVResource.h"
#include "AVUtility.h"
#include "PixelFormat.h"
#include "Containers/ResourceArray.h"

#include <atomic>

#define UE_API AVCODECSCORE_API

enum class EVideoFormat : uint8
{
	BGRA = EPixelFormat::PF_B8G8R8A8,
	ABGR10 = EPixelFormat::PF_A2B10G10R10,

	NV12 = EPixelFormat::PF_NV12,
	P010 = EPixelFormat::PF_P010,
	R8 = EPixelFormat::PF_R8,
	G16 = EPixelFormat::PF_G16,

	// TODO No match to engine formats and should be updated if that changes
	YUV420 = 253U,
	YUV444 = 254U,
	YUV444_16 = 255U,
};

inline bool operator==(EPixelFormat LHS, EVideoFormat RHS)
{
	return static_cast<uint8>(LHS) == static_cast<uint8>(RHS);
}

inline bool operator!=(EPixelFormat LHS, EVideoFormat RHS)
{
	return !(LHS == RHS);
}

inline bool operator==(EVideoFormat LHS, EPixelFormat RHS)
{
	return static_cast<uint8>(LHS) == static_cast<uint8>(RHS);
}

inline bool operator!=(EVideoFormat LHS, EPixelFormat RHS)
{
	return !(LHS == RHS);
}

/**
 * This struct defines how the allocated resource memory is used in the context of a video.
 */
struct FVideoDescriptor
{
public:
	/**
	 * Format of the pixel data.
	 */
	EVideoFormat Format;

	/**
	 * Width in pixels.
	 */
	uint32 Width;

	/**
	 * Height in pixels.
	 */
	uint32 Height;

	/**
	 * An implementation of the BulkData interface. Used for creating textures from existing data
	 */
	FResourceBulkDataInterface* BulkData;

	/**
	 * If this is a descriptor stored in a different format than it really is
	 * for transport then its raw descriptor should be described in this variable
	 */
	mutable FVideoDescriptor* RawDescriptor;

	FVideoDescriptor() = default;
	FVideoDescriptor(const FVideoDescriptor& Descriptor)
		: Format(Descriptor.Format)
		, Width(Descriptor.Width)
		, Height(Descriptor.Height)
		, BulkData(Descriptor.BulkData)
		, RawDescriptor(Descriptor.RawDescriptor ? new FVideoDescriptor(*Descriptor.RawDescriptor) : nullptr)
	{
	}

	FVideoDescriptor(EVideoFormat Format, uint32 Width, uint32 Height, FResourceBulkDataInterface* BulkData = nullptr)
		: Format(Format), Width(Width), Height(Height), BulkData(BulkData), RawDescriptor(nullptr)
	{
	}

	FVideoDescriptor(EVideoFormat Format, uint32 Width, uint32 Height, const FVideoDescriptor& RawDescriptor, FResourceBulkDataInterface* BulkData = nullptr)
		: Format(Format), Width(Width), Height(Height), BulkData(BulkData), RawDescriptor(new FVideoDescriptor(RawDescriptor))
	{
	}

	~FVideoDescriptor()
	{
		if (RawDescriptor)
		{
			delete RawDescriptor;
		}
	};

	bool operator==(FVideoDescriptor const& RHS) const
	{
		return Format == RHS.Format && Width == RHS.Width && Height == RHS.Height;
	}

	bool operator!=(FVideoDescriptor const& RHS) const
	{
		return !(*this == RHS);
	}

	// Planar formats are treated as single channel
	uint8 GetNumChannels() const
	{
		switch (Format)
		{
			case EVideoFormat::BGRA:
			case EVideoFormat::ABGR10:
				return 4;
			case EVideoFormat::YUV444:
			case EVideoFormat::YUV444_16:
			case EVideoFormat::NV12:
			case EVideoFormat::P010:
			case EVideoFormat::R8:
				return 1;
			default:
				return 0;
		}
	}

	// TODO(Nick): Try and remove this in favour of external initialisation
	uint32 GetSizeInBytes() const
	{
		const uint32 ChannelSize = Width * Height;
		switch (Format)
		{
			case EVideoFormat::BGRA:
			case EVideoFormat::ABGR10:
				return ChannelSize * 4;
			case EVideoFormat::YUV444:
				return ChannelSize * 3;
			case EVideoFormat::YUV444_16:
				return ChannelSize * 6;
			case EVideoFormat::NV12:
				return ChannelSize + (ChannelSize >> 1);
			case EVideoFormat::P010:
				return ChannelSize * 3;
			default:
				return 0;
		}
	}
};

/**
 * Base wrapper for a video device resource.
 */
class FVideoResource : public FAVResource
{
private:
	/**
	 * Descriptor of video data.
	 */
	FVideoDescriptor Descriptor;

	/**
	 * 
	 */
	std::atomic<bool> bIsInUse = false;

public:
	/**
	 * @return Get the descriptor of our video data in device memory.
	 */
	inline FVideoDescriptor const& GetRawDescriptor() const { return Descriptor.RawDescriptor ? *Descriptor.RawDescriptor : Descriptor; }

	/**
	 * @return Get the descriptor of our video data.
	 */
	inline FVideoDescriptor const& GetDescriptor() const { return Descriptor; }

	/**
	 * @return Get the format of our data.
	 */
	inline EVideoFormat GetFormat() const { return Descriptor.Format; }

	/**
	 * @return Get the width of our data.
	 */
	inline uint32 GetWidth() const { return Descriptor.Width; }

	/**
	 * @return Get the height of our data.
	 */
	inline uint32 GetHeight() const { return Descriptor.Height; }

	/**
	 * @return Get the Size in bytes of our Resource.
	 */
	inline uint32 GetSize() const { return Descriptor.GetSizeInBytes(); }

	/**
	 * 
	 */
	inline void SetUsing(bool bUsing) { bIsInUse = bUsing; }

	/**
	 * 
	 */
	inline bool IsInUse() { return bIsInUse; }

	UE_API FVideoResource(TSharedRef<FAVDevice> const& Device, FAVLayout const& Layout, FVideoDescriptor const& Descriptor);
	virtual ~FVideoResource() override = default;
};

/**
 * Convenience wrapper for a video device resource that requires a specific device context to function.
 */
template <typename TContext>
class TVideoResource : public FVideoResource
{
public:
	inline TSharedPtr<TContext> GetContext() const { return GetDevice()->template GetContext<TContext>(); }

	TVideoResource(TSharedRef<FAVDevice> const& Device, FAVLayout const& Layout, FVideoDescriptor const& Descriptor)
		: FVideoResource(Device, Layout, Descriptor)
	{
	}
};

/**
 * Wrapper for resolvable video resources.
 */
template <typename TResource>
using TResolvableVideoResource = TResolvable<TResource, TSharedPtr<FAVDevice>, FVideoDescriptor>;

/**
 * Wrapper for delegated resolvable video resources.
 */
template <typename TResource>
using TDelegatedVideoResource = TDelegated<TResource, TSharedPtr<FAVDevice>, FVideoDescriptor>;

/**
 * A simple pool-based resolvable video resource. The contents of the pool must be manually managed by the application.
 */
template <typename TResource>
class TPooledVideoResource : public TResolvableVideoResource<TResource>
{
public:
	TArray<TSharedPtr<TResource>> Pool;

	virtual bool IsResolved(TSharedPtr<FAVDevice> const& Device, FVideoDescriptor const& Descriptor) const override
	{
		return TResolvableVideoResource<TResource>::IsResolved(Device, Descriptor) && Pool.Contains(this->Get());
	}

protected:
	virtual TSharedPtr<TResource> TryResolve(TSharedPtr<FAVDevice> const& Device, FVideoDescriptor const& Descriptor) override
	{
		for (int i = 0; i < Pool.Num(); ++i)
		{
			if (Pool[i]->GetDevice() == Device && Pool[i]->GetDescriptor() == Descriptor)
			{
				return Pool[i];
			}
		}

		return nullptr;
	}
};

#undef UE_API
