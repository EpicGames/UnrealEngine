// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System;

public class AVCodecsCoreRHI : ModuleRules
{
	public AVCodecsCoreRHI(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		bLegacyPublicIncludePaths = false;
		DefaultBuildSettings = BuildSettingsVersion.Latest;
		
		PrivateDependencyModuleNames.AddRange(new string[] {
			"Engine",
			"AVCodecsCore",
			"RHI",
			"RHICore",
		});

		PublicDependencyModuleNames.AddRange(new string[] {
			"RenderCore",
			"Core",
			"CoreUObject"
		});
		
		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) || Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
				"Vulkan",
				"VulkanRHI"
			});
			
			AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan");
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"D3D11RHI",
				"D3D12RHI",
			});
			
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
        	    "MetalRHI",
        	});
	
        	PublicFrameworks.AddRange(new string[]{
        	    "AVFoundation",
        	    "VideoToolbox"
        	});
	
        	AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");	
		}
	}
}
