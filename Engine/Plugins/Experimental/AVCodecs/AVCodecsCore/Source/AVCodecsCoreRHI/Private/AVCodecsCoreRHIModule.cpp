// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "Misc/App.h"

#include "Video/Resources/VideoResourceRHI.h"
#include "Video/Decoders/VideoDecoderRHI.h"
#include "Video/Decoders/Configs/VideoDecoderConfigAV1.h"
#include "Video/Decoders/Configs/VideoDecoderConfigVP9.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH265.h"
#include "Video/Encoders/VideoEncoderRHI.h"
#include "Video/Encoders/Configs/VideoEncoderConfigAV1.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH265.h"

#if AVCODECS_USE_D3D
	#include "ID3D11DynamicRHI.h"
	#include "ID3D12DynamicRHI.h"
	#include "Video/Resources/D3D/VideoResourceD3D.h"
#endif 
#if AVCODECS_USE_VULKAN
	#include "IVulkanDynamicRHI.h"
	#include "Video/Resources/Vulkan/VideoResourceVulkan.h"
#endif
#if AVCODECS_USE_METAL
	#include "IMetalDynamicRHI.h"
	#include "Video/Resources/Metal/VideoResourceMetal.h"
#endif

#include "Engine/Engine.h"

class FAVCodecsCoreRHI : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		if (FApp::CanEverRender())
		{
			if (!GEngine || !GEngine->IsInitialized())
			{
				PostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddStatic(&FAVCodecsCoreRHI::PostEngineInit);
			}
			else
			{
				FAVCodecsCoreRHI::PostEngineInit();
			}
		}

		FVideoEncoder::Register<TVideoEncoderRHI<FVideoEncoderConfigAV1>, FVideoResourceRHI, FVideoEncoderConfigAV1>();
		FVideoEncoder::Register<TVideoEncoderRHI<FVideoEncoderConfigH264>, FVideoResourceRHI, FVideoEncoderConfigH264>();
		FVideoEncoder::Register<TVideoEncoderRHI<FVideoEncoderConfigH265>, FVideoResourceRHI, FVideoEncoderConfigH265>();

		FVideoDecoder::Register<TVideoDecoderRHI<FVideoDecoderConfigH264>, FVideoResourceRHI, FVideoDecoderConfigH264>();
		FVideoDecoder::Register<TVideoDecoderRHI<FVideoDecoderConfigH265>, FVideoResourceRHI, FVideoDecoderConfigH265>();
        FVideoDecoder::Register<TVideoDecoderRHI<FVideoDecoderConfigVP9>, FVideoResourceRHI, FVideoDecoderConfigVP9>();
		FVideoDecoder::Register<TVideoDecoderRHI<FVideoDecoderConfigAV1>, FVideoResourceRHI, FVideoDecoderConfigAV1>();
	}

	void ShutdownModule() override
	{
		// Remove the binding
		if (PostEngineInitHandle.IsValid())
		{
			FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle);
			PostEngineInitHandle.Reset();
		}
	}

	void static PostEngineInit()
	{
		// Remove the binding
		if (PostEngineInitHandle.IsValid())
		{
			FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle);
			PostEngineInitHandle.Reset();
		}

		switch (GDynamicRHI->GetInterfaceType())
		{
#if AVCODECS_USE_VULKAN
		case ERHIInterfaceType::Vulkan:
			FAVDevice::GetHardwareDevice()->SetContext<FVideoContextVulkan>(
				MakeShared<FVideoContextVulkan>(
					GetIVulkanDynamicRHI()->RHIGetVkInstance(),
					GetIVulkanDynamicRHI()->RHIGetVkDevice(),
					GetIVulkanDynamicRHI()->RHIGetVkPhysicalDevice(),
					[](char const* Name) -> PFN_vkVoidFunction {
						return (PFN_vkVoidFunction)GetIVulkanDynamicRHI()->RHIGetVkDeviceProcAddr(Name);
					}));

			break;
#endif
#if AVCODECS_USE_D3D
		case ERHIInterfaceType::D3D11:
			FAVDevice::GetHardwareDevice()->SetContext<FVideoContextD3D11>(
				MakeShared<FVideoContextD3D11>(
					GetID3D11DynamicRHI()->RHIGetDevice()));

			break;
		case ERHIInterfaceType::D3D12:
			FAVDevice::GetHardwareDevice()->SetContext<FVideoContextD3D12>(
				MakeShared<FVideoContextD3D12>(
					GetID3D12DynamicRHI()->RHIGetDevice(0)));

			break;
#endif
#if AVCODECS_USE_METAL
		case ERHIInterfaceType::Metal:
			FAVDevice::GetHardwareDevice()->SetContext<FVideoContextMetal>(
				MakeShared<FVideoContextMetal>(
					static_cast<MTL::Device*>(GDynamicRHI->RHIGetNativeDevice())));
			break;
#endif
		default:
			break;
		}

		FAVDevice::GetHardwareDevice()->SetContext<FVideoContextRHI>(MakeShared<FVideoContextRHI>());
	}

private:
	static FDelegateHandle PostEngineInitHandle;
};

FDelegateHandle FAVCodecsCoreRHI::PostEngineInitHandle;

IMPLEMENT_MODULE(FAVCodecsCoreRHI, AVCodecsCoreRHI);
