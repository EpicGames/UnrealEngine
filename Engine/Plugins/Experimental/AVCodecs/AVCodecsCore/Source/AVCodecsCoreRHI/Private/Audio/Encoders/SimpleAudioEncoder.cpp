// Copyright Epic Games, Inc. All Rights Reserved.

#include "Audio/Encoders/SimpleAudioEncoder.h"

#include "Audio/Encoders/Configs/AudioEncoderConfigAAC.h"
#include "Audio/Resources/AudioResourceCPU.h"

#include "HAL/RunnableThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleAudioEncoder)

uint32 USimpleAudioEncoder::Run()
{
	while (IsOpen() && IsAsync())
	{
		FAsyncFrame Frame;
		if (AsyncQueue.Dequeue(Frame))
		{
			this->Child->SendFrame(Frame.Resource, Frame.Timestamp);

			AsyncPool.Enqueue(Frame.Resource);
		}
	}

	return 0;
}

void USimpleAudioEncoder::Exit()
{
	AsyncThread = nullptr;

	AsyncQueue.Empty();
	AsyncPool.Empty();
}

bool USimpleAudioEncoder::IsAsync() const
{
	return AsyncThread != nullptr;
}

bool USimpleAudioEncoder::IsOpen() const
{
	return Child.IsValid() && Child->IsOpen();
}

bool USimpleAudioEncoder::Open(ESimpleAudioCodec Codec, FSimpleAudioEncoderConfig Config, bool bAsynchronous)
{
	if (IsOpen())
	{
		return false;
	}

	switch (Codec)
	{
	case ESimpleAudioCodec::AAC:
		{
			FAudioEncoderConfigAAC AVConfig;
			FAVExtension::TransformConfig<FAudioEncoderConfigAAC, FAudioEncoderConfig>(AVConfig, Config);
			
			Child = FAudioEncoder::Create<FAudioResourceCPU>(FAVDevice::GetHardwareDevice(), AVConfig);
		}

		break;
	}

	if (IsOpen() && bAsynchronous && FPlatformProcess::SupportsMultithreading())
	{
		AsyncThread = FRunnableThread::Create(this, TEXT("Simple Audio"));
	}

	return IsOpen();
}

void USimpleAudioEncoder::Close()
{
	if (IsOpen())
	{
		if (AsyncThread != nullptr)
		{
			AsyncThread->Kill();
			AsyncThread = nullptr;
		}

		Child.Reset();

		AsyncQueue.Empty();
		AsyncPool.Empty();
	}
}

bool USimpleAudioEncoder::SendFrameFloat(TArray<float> const& Resource, double Timestamp, int32 NumSamples, float SampleDuration)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"));
	}

	if (Resource.IsEmpty())
    {
    	return FAVResult(EAVResult::ErrorResolving, TEXT("Encoder resource is null"));
    }

	return SendFrame(Resource.GetData(), Timestamp, NumSamples, SampleDuration);
}

bool USimpleAudioEncoder::SendFrame(Audio::TSampleBuffer<float> const& Resource, double Timestamp)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"));
	}

	if (Resource.GetNumSamples() <= 0)
    {
    	return FAVResult(EAVResult::ErrorResolving, TEXT("Encoder resource is null"));
    }

	return SendFrame(Resource.GetData(), Timestamp, Resource.GetNumSamples(), Resource.GetSampleDuration());
}

bool USimpleAudioEncoder::SendFrame(float const* ResourceData, double Timestamp, int32 NumSamples, float SampleDuration)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"));
	}

	FAVLayout const ResourceLayout = FAVLayout(NumSamples, 0, NumSamples);
	FAudioDescriptor const ResourceDescriptor = FAudioDescriptor(NumSamples, SampleDuration);
	
	if (IsAsync())
	{
		TSharedPtr<FAudioResourceCPU> ResourceCPU = nullptr;
		while (!ResourceCPU.IsValid() && AsyncPool.Dequeue(ResourceCPU))
		{
			if (ResourceCPU->GetLayout() != ResourceLayout || ResourceCPU->GetDescriptor() != ResourceDescriptor)
			{
				ResourceCPU.Reset();
			}
		}

		if (!ResourceCPU.IsValid())
		{
			ResourceCPU = MakeShared<FAudioResourceCPU>(
				Child->GetDevice().ToSharedRef(),
				MakeShareable(new float[ResourceLayout.Size]),
				ResourceLayout,
				ResourceDescriptor);
		}

		FMemory::Memcpy(ResourceCPU->GetRaw().Get(), ResourceData, ResourceLayout.Size);

		AsyncQueue.Enqueue(
			FAsyncFrame(
				ResourceCPU,
				Timestamp * 1000));

		return true;
	}
	else
	{
		TSharedPtr<FAudioResourceCPU> const Resource = MakeShared<FAudioResourceCPU>(
				Child->GetDevice().ToSharedRef(),
				MakeShareable(new float[ResourceLayout.Size]),
				ResourceLayout,
				ResourceDescriptor);

		FMemory::Memcpy(Resource->GetRaw().Get(), ResourceData, ResourceLayout.Size);

		return this->Child->SendFrame(Resource, Timestamp * 1000);
	}
}

bool USimpleAudioEncoder::ReceivePacket(FSimpleAudioPacket& OutPacket)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"));
	}

	return Child->ReceivePacket(OutPacket.RawPacket);
}

void USimpleAudioEncoder::ReceivePackets(TArray<FSimpleAudioPacket>& OutPackets)
{
	if (IsOpen())
	{
		TArray<FAudioPacket> Packets;
		Child->ReceivePackets(Packets);

		OutPackets.Reserve(Packets.Num());
		for (int i = 0; i < Packets.Num(); ++i)
		{
			OutPackets.AddDefaulted_GetRef().RawPacket = Packets[i];
		}
	}
}

ESimpleAudioCodec USimpleAudioEncoder::GetCodec() const
{
	if (IsOpen())
	{
		return USimpleAudioHelper::GuessCodec(Child->GetInstance().ToSharedRef());
	}

	return ESimpleAudioCodec::AAC;
}

FSimpleAudioEncoderConfig USimpleAudioEncoder::GetConfig() const
{
	if (IsOpen())
	{
		return Child->GetMinimalConfig();
	}

	return FSimpleAudioEncoderConfig();
}

void USimpleAudioEncoder::SetConfig(FSimpleAudioEncoderConfig NewConfig)
{
	if (IsOpen())
	{
		Child->SetMinimalConfig(NewConfig);
	}
}
