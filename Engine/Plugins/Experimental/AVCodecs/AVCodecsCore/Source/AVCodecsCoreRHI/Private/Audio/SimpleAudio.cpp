// Copyright Epic Games, Inc. All Rights Reserved.

#include "Audio/SimpleAudio.h"

#include "Audio/Encoders/Configs/AudioEncoderConfigAAC.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleAudio)

ESimpleAudioCodec USimpleAudioHelper::GuessCodec(TSharedRef<FAVInstance> const& Instance)
{
	if (Instance->Has<FAudioEncoderConfigAAC>())
	{
		return ESimpleAudioCodec::AAC;
	}

	return ESimpleAudioCodec::AAC;
}
