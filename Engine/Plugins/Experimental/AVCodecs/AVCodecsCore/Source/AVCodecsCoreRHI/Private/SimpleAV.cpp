// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleAV.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleAV)

EAVPreset USimpleAVHelper::ConvertPreset(ESimpleAVPreset From)
{
	switch (From)
	{
	case ESimpleAVPreset::UltraLowQuality:
		return EAVPreset::UltraLowQuality;
	case ESimpleAVPreset::LowQuality:
		return EAVPreset::LowQuality;
	case ESimpleAVPreset::HighQuality:
		return EAVPreset::HighQuality;
	case ESimpleAVPreset::Lossless:
		return EAVPreset::Lossless;
	default:
		return EAVPreset::Default;
	}
}
