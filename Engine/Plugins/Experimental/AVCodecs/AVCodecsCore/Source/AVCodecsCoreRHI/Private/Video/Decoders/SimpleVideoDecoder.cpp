// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Decoders/SimpleVideoDecoder.h"

#include "Engine/TextureRenderTarget2D.h"
#include "TextureResource.h"
#include "HAL/RunnableThread.h"

#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH265.h"
#include "Video/Resources/VideoResourceRHI.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleVideoDecoder)

uint32 USimpleVideoDecoder::Run()
{
	while (IsOpen() && IsAsync())
	{
		FSimpleVideoPacket Packet;
		if (AsyncQueue.Dequeue(Packet))
		{
			this->Child->SendPacket(Packet.RawPacket);
		}
	}

	return 0;
}

void USimpleVideoDecoder::Exit()
{
	AsyncThread = nullptr;

	AsyncQueue.Empty();
}

bool USimpleVideoDecoder::IsAsync() const
{
	return AsyncThread != nullptr;
}

bool USimpleVideoDecoder::IsOpen() const
{
	return Child.IsValid() && Child->IsOpen();
}

bool USimpleVideoDecoder::Open(ESimpleVideoCodec Codec, bool bAsynchronous)
{
	if (IsOpen())
	{
		return false;
	}

	switch (Codec)
	{
	case ESimpleVideoCodec::H264:
		Child = FVideoDecoder::Create<FVideoResourceRHI>(FAVDevice::GetHardwareDevice(), FVideoDecoderConfigH264());

		break;
	case ESimpleVideoCodec::H265:
		Child = FVideoDecoder::Create<FVideoResourceRHI>(FAVDevice::GetHardwareDevice(), FVideoDecoderConfigH265());

		break;
	}
	
	if (IsOpen() && bAsynchronous && FPlatformProcess::SupportsMultithreading())
	{
		AsyncThread = FRunnableThread::Create(this, TEXT("Simple Video"));
	}

	return IsOpen();
}

void USimpleVideoDecoder::Close()
{
	if (IsOpen())
	{
		if (AsyncThread != nullptr)
		{
			AsyncThread->Kill();
			AsyncThread = nullptr;
		}

		Child.Reset();

		AsyncQueue.Empty();
	}
}

bool USimpleVideoDecoder::SendPacket(FSimpleVideoPacket const& Packet)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"));
	}

	if (IsAsync())
	{
		AsyncQueue.Enqueue(Packet);

		return true;
	}
	else
	{
		return this->Child->SendPacket(Packet.RawPacket);
	}
}

bool USimpleVideoDecoder::ReceiveFrame(UTextureRenderTarget2D* Resource)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"));
	}

	if (Resource == nullptr)
	{
		return FAVResult(EAVResult::ErrorResolving, TEXT("Decoder resource is null"));
	}

	USimpleVideoHelper::ShareRenderTarget2D(Resource);

	TDelegatedVideoResource<FVideoResourceRHI> WrappedResource([&Resource](TSharedPtr<FVideoResourceRHI>& InOutResult, TSharedPtr<FAVDevice> const& InDevice, FVideoDescriptor const& Descriptor)
	{
		FRHITextureDesc const& RawDesc = Resource->GetResource()->GetTexture2DRHI()->GetDesc();

		/*if (RawDesc.Format != LayoutFormat)
		{
			Resource->InitCustomFormat(Layout.Extent.X, Layout.Extent.Y, LayoutFormat, false);
		}
		else */if (RawDesc.GetSize().X != Descriptor.Width || RawDesc.GetSize().Y != Descriptor.Height)
		{
			Resource->ResizeTarget(Descriptor.Width , Descriptor.Height);
		}

//TODO-TE
		InOutResult = MakeShared<FVideoResourceRHI>(InDevice.ToSharedRef(), FVideoResourceRHI::FRawData{ Resource->GetResource()->GetTexture2DRHI(), nullptr, 0 });
	});

	return Child->ReceiveFrame(WrappedResource);
}

ESimpleVideoCodec USimpleVideoDecoder::GetCodec() const
{
	if (IsOpen())
	{
		return USimpleVideoHelper::GuessCodec(Child->GetInstance().ToSharedRef());
	}

	return ESimpleVideoCodec::H264;
}
