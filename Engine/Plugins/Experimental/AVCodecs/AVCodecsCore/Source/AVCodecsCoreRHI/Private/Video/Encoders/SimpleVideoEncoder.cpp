// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Encoders/SimpleVideoEncoder.h"

#include "Engine/TextureRenderTarget2D.h"
#include "Engine/Texture2D.h"
#include "TextureResource.h"
#include "RHI.h"
#include "HAL/RunnableThread.h"

#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH265.h"
#include "Video/Resources/VideoResourceRHI.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleVideoEncoder)


uint32 USimpleVideoEncoder::Run()
{
	while (IsOpen() && IsAsync())
	{
		FAsyncFrame Frame;
		if (AsyncQueue.Dequeue(Frame))
		{
			this->Child->SendFrame(Frame.Resource, Frame.Timestamp, Frame.bForceKeyframe);

			AsyncPool.Enqueue(Frame.Resource);
		}
	}

	return 0;
}

void USimpleVideoEncoder::Exit()
{
	AsyncThread = nullptr;

	AsyncQueue.Empty();
	AsyncPool.Empty();
}

bool USimpleVideoEncoder::IsAsync() const
{
	return AsyncThread != nullptr;
}

bool USimpleVideoEncoder::IsOpen() const
{
	return Child.IsValid() && Child->IsOpen();
}

bool USimpleVideoEncoder::Open(ESimpleVideoCodec Codec, FSimpleVideoEncoderConfig Config, bool bAsynchronous)
{
	if (IsOpen())
	{
		return false;
	}

	switch (Codec)
	{
	case ESimpleVideoCodec::H264:
		{
			FVideoEncoderConfigH264 AVConfig;
			FAVExtension::TransformConfig<FVideoEncoderConfigH264, FVideoEncoderConfig>(AVConfig, Config);
			
			Child = FVideoEncoder::Create<FVideoResourceRHI>(FAVDevice::GetHardwareDevice(), AVConfig);
		}

		break;
	case ESimpleVideoCodec::H265:
		{
			FVideoEncoderConfigH265 AVConfig;
			FAVExtension::TransformConfig<FVideoEncoderConfigH265, FVideoEncoderConfig>(AVConfig, Config);
			
			Child = FVideoEncoder::Create<FVideoResourceRHI>(FAVDevice::GetHardwareDevice(), AVConfig);
		}

		break;
	}

	if (IsOpen() && bAsynchronous && FPlatformProcess::SupportsMultithreading())
	{
		AsyncThread = FRunnableThread::Create(this, TEXT("Simple Video"));
	}

	return IsOpen();
}

void USimpleVideoEncoder::Close()
{
	if (IsOpen())
	{
		if (AsyncThread != nullptr)
		{
			AsyncThread->Kill();
			AsyncThread = nullptr;
		}

		Child.Reset();

		AsyncQueue.Empty();
		AsyncPool.Empty();
	}
}

bool USimpleVideoEncoder::SendFrameRenderTarget(UTextureRenderTarget2D* Resource, double Timestamp, bool bForceKeyframe)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"));
	}

	if (Resource == nullptr)
	{
		return FAVResult(EAVResult::ErrorResolving, TEXT("Encoder resource is null"));
	}

	USimpleVideoHelper::ShareRenderTarget2D(Resource);

	return SendFrame(Resource->GetResource()->GetTexture2DRHI(), Timestamp, bForceKeyframe);
}

bool USimpleVideoEncoder::SendFrameTexture(UTexture2D* Resource, double Timestamp, bool bForceKeyframe)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"));
	}

	if (Resource == nullptr)
	{
		return FAVResult(EAVResult::ErrorResolving, TEXT("Encoder resource is null"));
	}

	return SendFrame(Resource->GetResource()->GetTexture2DRHI(), Timestamp, bForceKeyframe);
}

bool USimpleVideoEncoder::SendFrame(FTextureRHIRef const& Resource, double Timestamp, bool bForceKeyframe)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"));
	}

	if (IsAsync())
	{
		FAVLayout const ResourceLayout = FVideoResourceRHI::GetLayoutFrom(Child->GetDevice().ToSharedRef(), Resource);
		FVideoDescriptor const ResourceDescriptor = FVideoResourceRHI::GetDescriptorFrom(Child->GetDevice().ToSharedRef(), Resource);

		TSharedPtr<FVideoResourceRHI> ResourceRHI = nullptr;
		while (!ResourceRHI.IsValid() && AsyncPool.Dequeue(ResourceRHI))
		{
			if (ResourceRHI->GetLayout() != ResourceLayout || ResourceRHI->GetDescriptor() != ResourceDescriptor)
			{
				ResourceRHI.Reset();
			}
		}

		if (!ResourceRHI.IsValid())
		{
			ResourceRHI = FVideoResourceRHI::Create(Child->GetDevice().ToSharedRef(), ResourceDescriptor);
		}

		ResourceRHI->CopyFrom(Resource);

		AsyncQueue.Enqueue(FAsyncFrame(ResourceRHI, Timestamp * 1000, bForceKeyframe));

		return true;
	}
	else
	{
//TODO-TE
		TSharedPtr<FVideoResourceRHI> const ResourceRHI = MakeShared<FVideoResourceRHI>(
			Child->GetDevice().ToSharedRef(),
			FVideoResourceRHI::FRawData{ Resource, nullptr, 0 });

		return this->Child->SendFrame(ResourceRHI, Timestamp * 1000, bForceKeyframe);
	}
}

bool USimpleVideoEncoder::ReceivePacket(FSimpleVideoPacket& OutPacket)
{
	if (!IsOpen())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"));
	}

	return Child->ReceivePacket(OutPacket.RawPacket);
}

void USimpleVideoEncoder::ReceivePackets(TArray<FSimpleVideoPacket>& OutPackets)
{
	if (IsOpen())
	{
		TArray<FVideoPacket> Packets;
		Child->ReceivePackets(Packets);

		OutPackets.Reserve(Packets.Num());
		for (int i = 0; i < Packets.Num(); ++i)
		{
			OutPackets.AddDefaulted_GetRef().RawPacket = Packets[i];
		}
	}
}

ESimpleVideoCodec USimpleVideoEncoder::GetCodec() const
{
	if (IsOpen())
	{
		return USimpleVideoHelper::GuessCodec(Child->GetInstance().ToSharedRef());
	}

	return ESimpleVideoCodec::H264;
}

FSimpleVideoEncoderConfig USimpleVideoEncoder::GetConfig() const
{
	if (IsOpen())
	{
		return Child->GetMinimalConfig();
	}

	return FSimpleVideoEncoderConfig();
}

void USimpleVideoEncoder::SetConfig(FSimpleVideoEncoderConfig NewConfig)
{
	if (IsOpen())
	{
		Child->SetMinimalConfig(NewConfig);
	}
}
