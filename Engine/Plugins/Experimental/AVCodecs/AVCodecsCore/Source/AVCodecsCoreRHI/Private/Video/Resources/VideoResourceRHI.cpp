// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/Resources/VideoResourceRHI.h"

#include "Video/Resources/VideoResourceCPU.h"

#if AVCODECS_USE_D3D
	#include "ID3D11DynamicRHI.h"
	#include "ID3D12DynamicRHI.h"
	#include "D3D12RHI.h"
	#include "BoundShaderStateCache.h"
	#include "D3D12ShaderResources.h"
	#include "Video/Resources/D3D/VideoResourceD3D.h"
#endif 
#if AVCODECS_USE_VULKAN
	#include "IVulkanDynamicRHI.h"
	#include "Video/Resources/Vulkan/VideoResourceVulkan.h"
#endif
#if AVCODECS_USE_METAL
	#include "Video/Resources/Metal/VideoResourceMetal.h"
#endif

#include "Async/Async.h"
#include "CoreGlobals.h"
#include "HAL/Event.h"
#include "RHIStaticStates.h"
#include "MediaShaders.h"
#include "PipelineStateCache.h"
#include "ColorManagement/ColorManagementDefines.h"
#include "ColorManagement/ColorSpace.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

REGISTER_TYPEID(FVideoContextRHI);
REGISTER_TYPEID(FVideoResourceRHI);

#define CFSafeRelease(x)                \
    if (x != nullptr)                   \
    {                                   \
        CFRelease(x);                   \
        x = nullptr;                    \
    }

FAVLayout FVideoResourceRHI::GetLayoutFrom(TSharedRef<FAVDevice> const& Device, FTextureRHIRef const& Raw)
{
	switch (GDynamicRHI->GetInterfaceType())
	{
#if AVCODECS_USE_VULKAN
	case ERHIInterfaceType::Vulkan:
		{
			FRHITextureDesc const& Desc = Raw->GetDesc();
			EPixelFormat const PixelFormat = Raw->GetDesc().Format;
			uint32 const BlockSizeX = GPixelFormats[PixelFormat].BlockSizeX;
			uint32 NumBlocksX = (Desc.Extent.X + BlockSizeX - 1) / BlockSizeX;

			if (PixelFormat == PF_PVRTC2 || PixelFormat == PF_PVRTC4)
			{
				// PVRTC has minimum 2 blocks width
				NumBlocksX = FMath::Max<uint32>(NumBlocksX, 2);
			}

			uint32 const BlockBytes = GPixelFormats[PixelFormat].BlockBytes;

			FVulkanRHIAllocationInfo const TextureAllocationInfo = GetIVulkanDynamicRHI()->RHIGetAllocationInfo(Raw.GetReference());

			return FAVLayout(NumBlocksX * BlockBytes, TextureAllocationInfo.Offset, TextureAllocationInfo.Size);
		}
		
#endif
#if AVCODECS_USE_D3D
	case ERHIInterfaceType::D3D11:
		{
			// This is a guess because D3D11 doesn't expose the actual allocation size
			int64 const Size = GetID3D11DynamicRHI()->RHIGetResourceMemorySize(Raw);

			uint32 const BlockSizeX = GPixelFormats[Raw->GetFormat()].BlockSizeX;
			uint32 const BlockBytes = GPixelFormats[Raw->GetFormat()].BlockBytes;
			uint32 const NumBlocksX = (Raw->GetSizeX() + BlockSizeX - 1) / BlockSizeX;

			// TODO (Andrew) Actually get the offset, may not be possible
			return FAVLayout(NumBlocksX * BlockBytes, 0, Size);
		}
		
	case ERHIInterfaceType::D3D12:
		{
			TRefCountPtr<ID3D12Device4> AdvancedDevice;

			HRESULT const Result = Device->GetContext<FVideoContextD3D12>()->Device->QueryInterface(AdvancedDevice.GetInitReference());
			if (FAILED(Result))
			{
				FAVResult::Log(EAVResult::ErrorMapping, TEXT("D3D12 version is outdated"), TEXT("D3D12"), Result);

				break;
			}

			D3D12_RESOURCE_DESC const& RawDesc = GetID3D12DynamicRHI()->RHIGetResource(Raw)->GetDesc();
			D3D12_RESOURCE_ALLOCATION_INFO1 RawInfo;

			AdvancedDevice->GetResourceAllocationInfo1(0, 1, &RawDesc, &RawInfo);

			uint32 const BlockSizeX = GPixelFormats[Raw->GetFormat()].BlockSizeX;
			uint32 const BlockBytes = GPixelFormats[Raw->GetFormat()].BlockBytes;
			uint32 const NumBlocksX = (Raw->GetSizeX() + BlockSizeX - 1) / BlockSizeX;

			return FAVLayout(Align(NumBlocksX * BlockBytes, FD3D12_TEXTURE_DATA_PITCH_ALIGNMENT), RawInfo.Offset, RawInfo.SizeInBytes);
		}
		
#endif
#if AVCODECS_USE_METAL
        case ERHIInterfaceType::Metal:
            {
                uint32 const Size =  GDynamicRHI->RHIComputeMemorySize(Raw);

                uint32 const BlockSizeX = GPixelFormats[Raw->GetFormat()].BlockSizeX;
                uint32 const BlockBytes = GPixelFormats[Raw->GetFormat()].BlockBytes;
                uint32 const NumBlocksX = (Raw->GetSizeX() + BlockSizeX - 1) / BlockSizeX;

                // TODO (Andrew) Actually get the offset, may not be possible
                return FAVLayout(NumBlocksX * BlockBytes, 0, Size);
            }
            
#endif
	default:
		break;
	}

	return FAVLayout();
}

FVideoDescriptor FVideoResourceRHI::GetDescriptorFrom(TSharedRef<FAVDevice> const& Device, FTextureRHIRef const& Raw)
{
	FRHITextureDesc const& RawDesc = Raw->GetDesc();
			return FVideoDescriptor(static_cast<EVideoFormat>(RawDesc.Format), RawDesc.Extent.X, RawDesc.Extent.Y);
}

TSharedPtr<FVideoResourceRHI> FVideoResourceRHI::Create(TSharedPtr<FAVDevice> const& Device, FVideoDescriptor const& Descriptor, ETextureCreateFlags AdditionalFlags, bool bIsSRGB)
{
	if (Device->HasContext<FVideoContextRHI>())
	{
		FRHITextureCreateDesc TextureDesc = FRHITextureCreateDesc::Create2D(TEXT("AVCodecs Resource"), Descriptor.Width, Descriptor.Height, static_cast<EPixelFormat>(Descriptor.Format));

		TextureDesc.SetClearValue(FClearValueBinding::None);
#if AVCODECS_USE_METAL
		TextureDesc.SetFlags(ETextureCreateFlags::CPUReadback);
		TextureDesc.SetInitialState(ERHIAccess::CPURead);
#else
    	TextureDesc.SetFlags(ETextureCreateFlags::RenderTargetable);
		TextureDesc.SetInitialState(ERHIAccess::Present);
#endif

		TextureDesc.SetNumMips(1);

		if (RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
		{
			TextureDesc.AddFlags(ETextureCreateFlags::External);
		}
		else
		{
			TextureDesc.AddFlags(ETextureCreateFlags::Shared);
		}

		TextureDesc.AddFlags(AdditionalFlags);
        
        if (Descriptor.BulkData)
        {
            TextureDesc.SetInitActionBulkData(Descriptor.BulkData);
        }

		// Unreals support for NV12 asnd P010 is not universally supported so we actually use R8 or G16 under the hood
		switch (Descriptor.Format)
		{
		// 420 sampled semiplanar so: 1 x Y for every sample, 0.25 U for every sample, 0.25 V for every sample = 1.5x Y axis
		case EVideoFormat::NV12:
			TextureDesc.Format = EPixelFormat::PF_R8;
			TextureDesc.Extent.Y *= 1.5;
			Descriptor.RawDescriptor = new FVideoDescriptor(EVideoFormat::R8, TextureDesc.Extent.X, TextureDesc.Extent.Y);
			break;
		case EVideoFormat::P010:
			TextureDesc.Format = EPixelFormat::PF_G16;
			TextureDesc.Extent.Y *= 1.5;
			Descriptor.RawDescriptor = new FVideoDescriptor(EVideoFormat::G16, TextureDesc.Extent.X, TextureDesc.Extent.Y);
			break;
		// 444 sampled planar so: 1 x Y for every sample, 1 x U for every sample, 1 x V for every sample = 3x X axis
		case EVideoFormat::YUV444:
			TextureDesc.Format = EPixelFormat::PF_R8;
			TextureDesc.Extent.X *= 3;
			Descriptor.RawDescriptor = new FVideoDescriptor(EVideoFormat::R8, TextureDesc.Extent.X, TextureDesc.Extent.Y);
			break;
		case EVideoFormat::YUV444_16:
			TextureDesc.Format = EPixelFormat::PF_G16;
			TextureDesc.Extent.X *= 3;
			Descriptor.RawDescriptor = new FVideoDescriptor(EVideoFormat::G16, TextureDesc.Extent.X, TextureDesc.Extent.Y);
			break;
        case EVideoFormat::BGRA:
			TextureDesc.Format = EPixelFormat::PF_B8G8R8A8;
			Descriptor.RawDescriptor = new FVideoDescriptor(EVideoFormat::BGRA, TextureDesc.Extent.X, TextureDesc.Extent.Y);
			// Only BGRA can be directly displayed and could need sRGB adjustment, setting this for all formats triggers a check		
			// in VulkanRHI
			if (bIsSRGB)
			{
				TextureDesc.AddFlags(ETextureCreateFlags::SRGB);
			}
			break;
		default:
			break;
		}

		// There seems to be an intermitant bug with VulkanRHI that results in crahes if we dont enque the allocation 
		// of this texture
		FTextureRHIRef Texture;
		if (IsInRenderingThread())
		{
			Texture = FRHICommandListImmediate::Get().CreateTexture(TextureDesc);
		}
		else
		{
			// We have to add a fence here as VulkanRHI adds a fill to the command buffer that might not be dispatched until
			// after a external decoder API has already copied to the texture. If we had access to Signal Semaphores they could
			// instead be shared to the other APIs instead.
			FGPUFenceRHIRef Fence = GDynamicRHI->RHICreateGPUFence("CreateVideoResourceRHI");
					
			ENQUEUE_RENDER_COMMAND(CreateTexture)
				([&Texture, TextureDesc, Fence](FRHICommandListImmediate& RHICmdList) {
					Texture = RHICmdList.CreateTexture(TextureDesc);
					RHICmdList.WriteGPUFence(Fence);
				});

			while (!Fence->Poll()) {
				constexpr float SleepTimeSeconds = 50 * 1E-6f;
				FPlatformProcess::SleepNoStats(SleepTimeSeconds);
			}
		}

		return MakeShareable(new FVideoResourceRHI(Device.ToSharedRef(), { Texture, nullptr, 0 }, Descriptor));
	}

	return nullptr;
}

FVideoResourceRHI::FVideoResourceRHI(TSharedRef<FAVDevice> const& Device, FRawData const& Raw, FVideoDescriptor const& OverrideDescriptor)
	: TVideoResource(Device, GetLayoutFrom(Device, Raw.Texture), OverrideDescriptor)
	, Raw(Raw)
{
#if AVCODECS_USE_D3D
	if (RHIGetInterfaceType() == ERHIInterfaceType::D3D12)
	{
		// Create a fence if we did not pass one in
		if (!this->Raw.Fence.IsValid())
		{
			Device->GetContext<FVideoContextD3D12>()->Device->CreateFence(0, D3D12_FENCE_FLAG_SHARED, IID_PPV_ARGS(this->Raw.Fence.GetInitReference()));
		}
	}
#endif

}

FVideoResourceRHI::FVideoResourceRHI(TSharedRef<FAVDevice> const& Device, FRawData const& Raw)
	: FVideoResourceRHI(Device, Raw, GetDescriptorFrom(Device, Raw.Texture)) 
{
}

FAVResult FVideoResourceRHI::Validate() const
{
	if (!Raw.Texture.IsValid())
	{
		return FAVResult(EAVResult::ErrorInvalidState, TEXT("Raw resource is invalid"), TEXT("RHI"));
	}

	FRHITextureDesc const& RawDesc = Raw.Texture->GetDesc();
	if (RawDesc.Format == GetFormat() || RawDesc.GetSize().X != GetWidth() || RawDesc.GetSize().Y != GetHeight())
	{
		return FAVResult(EAVResult::WarningInvalidState, TEXT("Raw resource differs from wrapper"), TEXT("RHI"));
	}

	return EAVResult::Success;
}

void FVideoResourceRHI::Lock()
{
	TVideoResource::Lock();
}

FScopeLock FVideoResourceRHI::LockScope()
{
	return TVideoResource::LockScope();
}

void FVideoResourceRHI::CopyFrom(TArrayView64<uint8> const& From)
{
	ENQUEUE_RENDER_COMMAND(WriteRHITextureData)
		([this, &From](FRHICommandListImmediate& RHICmdList) {
			// TODO (aidan.possemiers) probably need a transition here to do the update
			FUpdateTextureRegion2D UpdateRegion = FUpdateTextureRegion2D(0, 0, 0, 0, this->GetRawDescriptor().Width, this->GetRawDescriptor().Height);
			RHICmdList.UpdateTexture2D(this->GetRaw().Texture, 0, UpdateRegion, this->GetRawDescriptor().Width, From.GetData());
		});
}

void FVideoResourceRHI::CopyFrom(TArrayView<uint8> const& From)
{
	this->CopyFrom(TArrayView64<uint8>(const_cast<uint8*>(From.GetData()), From.Num()));
}

void FVideoResourceRHI::CopyFrom(TArray64<uint8> const& From)
{
	this->CopyFrom(TArrayView64<uint8>(const_cast<uint8*>(From.GetData()), From.Num()));
}

void FVideoResourceRHI::CopyFrom(TArray<uint8> const& From)
{
	this->CopyFrom(TArrayView64<uint8>(const_cast<uint8*>(From.GetData()), From.Num()));
}

void FVideoResourceRHI::CopyFrom(FTextureRHIRef const& From)
{
	ENQUEUE_RENDER_COMMAND(CopyRHITextureData)
	([this, &From](FRHICommandListImmediate& RHICmdList) {
    		RHICmdList.Transition(FRHITransitionInfo(From, ERHIAccess::Unknown, ERHIAccess::CopySrc));
    		RHICmdList.Transition(FRHITransitionInfo(this->GetRaw().Texture, ERHIAccess::Unknown, ERHIAccess::CopyDest));

    		RHICmdList.CopyTexture(From, this->GetRaw().Texture, FRHICopyTextureInfo());

    		RHICmdList.Transition(FRHITransitionInfo(From, ERHIAccess::CopySrc, ERHIAccess::Present));
    	});
}

TSharedPtr<FVideoResourceRHI> FResolvableVideoResourceRHI::TryResolve(TSharedPtr<FAVDevice> const& Device, FVideoDescriptor const& Descriptor)
{
	return FVideoResourceRHI::Create(Device, Descriptor, AdditionalFlags);
}

TSharedPtr<FVideoResourceRHI> FVideoResourceRHI::TransformResource(FVideoDescriptor const& OutDescriptor)
{
	TSharedPtr<FVideoResourceRHI> OutResource;

	if (Raw.Texture.IsValid())
	{
		OutResource = FVideoResourceRHI::Create(GetDevice(), OutDescriptor);

		if (GetDescriptor() == OutResource->GetDescriptor())
		{
			OutResource->CopyFrom(Raw.Texture);
		}
		else
		{
			// TODO (aidan.possemiers) do some compute shader magic here to transform the resources using our supported types
			if (GetFormat() == EVideoFormat::NV12 && OutResource->GetFormat() == EVideoFormat::BGRA)
			{
				ENQUEUE_RENDER_COMMAND(TransformNV12toRGBA)
				([Source = GetRaw().Texture, Dest = OutResource->GetRaw().Texture](FRHICommandListImmediate& RHICmdList) {

					SCOPED_DRAW_EVENT(RHICmdList, FVideoResourceRHI_NV12toBGRA);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.Transition(FRHITransitionInfo(Dest, ERHIAccess::Unknown, ERHIAccess::RTV));

					FIntPoint OutputDim(Dest->GetDesc().Extent.X, Dest->GetDesc().Extent.Y);

					RHICmdList.Transition(FRHITransitionInfo(Source, ERHIAccess::Unknown, ERHIAccess::RTV));

					FRHIRenderPassInfo RPInfo(Dest, ERenderTargetActions::DontLoad_Store);
					RHICmdList.BeginRenderPass(RPInfo, TEXT("TransformVideoResource"));
					{
						RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
						RHICmdList.SetViewport(0, 0, 0.0f, (float)OutputDim.X, (float)OutputDim.Y, 1.0f);

						GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
						GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
						GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
						GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

						// configure media shaders
						auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
						TShaderMapRef<FMediaShadersVS> VertexShader(ShaderMap);

						GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
						GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

						// TODO (aidan.possemiers) do we need get the not scaled version of this matrix
						FMatrix PreMtx = FMatrix::Identity;
						PreMtx.M[0][3] = -MediaShaders::YUVOffset8bits.X;
						PreMtx.M[1][3] = -MediaShaders::YUVOffset8bits.Y;
						PreMtx.M[2][3] = -MediaShaders::YUVOffset8bits.Z;
						auto YUVToRGBMatrix = FMatrix44f(MediaShaders::YuvToRgbRec709Scaled * PreMtx);

						TShaderMapRef<FNV12ConvertAsBytesPS> ConvertShader(ShaderMap);
						GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
						SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
						SetShaderParametersLegacyPS(RHICmdList, ConvertShader, Source, OutputDim, YUVToRGBMatrix, UE::Color::EEncoding::sRGB, FMatrix44f::Identity, false);

						// draw full size quad into render target
						FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(RHICmdList);
						RHICmdList.SetStreamSource(0, VertexBuffer, 0);
						// set viewport to RT size
						RHICmdList.SetViewport(0, 0, 0.0f, (float)OutputDim.X, (float)OutputDim.Y, 1.0f);

						RHICmdList.DrawPrimitive(0, 2, 1);
					}					
					RHICmdList.EndRenderPass();

					RHICmdList.Transition(FRHITransitionInfo(Dest, ERHIAccess::RTV, ERHIAccess::SRVGraphics));

					});
			}
			else 
			{
				unimplemented();
			}
		}
	}

	return OutResource;
}

void FVideoResourceRHI::TransformResourceTo(FRHICommandListImmediate& RHICmdList, FTextureRHIRef Target, FVideoDescriptor const& OutDescriptor)
{
	if (Raw.Texture.IsValid())
	{
		if (GetFormat() == EVideoFormat::NV12 && Target->GetDesc().Format == EVideoFormat::BGRA)
		{
				SCOPED_DRAW_EVENT(RHICmdList, FVideoResourceRHI_NV12toBGRA);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.Transition(FRHITransitionInfo(Target, ERHIAccess::Unknown, ERHIAccess::RTV));

				FIntPoint OutputDim(Target->GetDesc().Extent.X, Target->GetDesc().Extent.Y);

				RHICmdList.Transition(FRHITransitionInfo(Raw.Texture, ERHIAccess::Unknown, ERHIAccess::RTV));

				FRHIRenderPassInfo RPInfo(Target, ERenderTargetActions::DontLoad_Store);
				RHICmdList.BeginRenderPass(RPInfo, TEXT("TransformVideoResource"));
				{
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					RHICmdList.SetViewport(0, 0, 0.0f, (float)OutputDim.X, (float)OutputDim.Y, 1.0f);

					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
					GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
					GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

					// configure media shaders
					auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
					TShaderMapRef<FMediaShadersVS> VertexShader(ShaderMap);

					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

					// TODO (aidan.possemiers) do we need get the not scaled version of this matrix
					FMatrix PreMtx = FMatrix::Identity;
					PreMtx.M[0][3] = -MediaShaders::YUVOffset8bits.X;
					PreMtx.M[1][3] = -MediaShaders::YUVOffset8bits.Y;
					PreMtx.M[2][3] = -MediaShaders::YUVOffset8bits.Z;
					auto YUVToRGBMatrix = FMatrix44f(MediaShaders::YuvToRgbRec709Scaled * PreMtx);

					TShaderMapRef<FNV12ConvertAsBytesPS> ConvertShader(ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					SetShaderParametersLegacyPS(RHICmdList, ConvertShader, Raw.Texture, OutputDim, YUVToRGBMatrix, UE::Color::EEncoding::sRGB, FMatrix44f::Identity, false);

					// draw full size quad into render target
					FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(RHICmdList);
					RHICmdList.SetStreamSource(0, VertexBuffer, 0);
					// set viewport to RT size
					RHICmdList.SetViewport(0, 0, 0.0f, (float)OutputDim.X, (float)OutputDim.Y, 1.0f);

					RHICmdList.DrawPrimitive(0, 2, 1);
				}
				RHICmdList.EndRenderPass();

				RHICmdList.Transition(FRHITransitionInfo(Target, ERHIAccess::RTV, ERHIAccess::SRVGraphics));
		}
		else
		{
			unimplemented();
		}
	}
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformResource(TSharedPtr<FVideoResourceRHI>& OutResource, TSharedPtr<FVideoResourceCPU> const& InResource)
{
	if (InResource.IsValid())
	{
		if (InResource->GetDevice()->HasContext<FVideoContextCPU>())
		{
			if (!OutResource.IsValid())
			{
				OutResource = FVideoResourceRHI::Create(InResource->GetDevice(), InResource->GetDescriptor());
			}

			OutResource->CopyFrom(TArrayView<uint8>(InResource->GetRaw().Get(), InResource->GetSize()));

			return OutResource->Validate();
		}

		return FAVResult(EAVResult::ErrorMapping, TEXT("No CPU context found"), TEXT("RHI"));
	}

	return FAVResult(EAVResult::ErrorMapping, TEXT("Input resource is not valid"), TEXT("RHI"));
}

#if AVCODECS_USE_D3D

template <>
DLLEXPORT FAVResult FAVExtension::TransformResource(TSharedPtr<FVideoResourceD3D11>& OutResource, TSharedPtr<FVideoResourceRHI> const& InResource)
{
	if (InResource.IsValid())
	{
		if (InResource->GetDevice()->HasContext<FVideoContextD3D11>())
		{
			TRefCountPtr<ID3D11Texture2D> InResourceRaw = nullptr;

			HRESULT const Result = GetID3D11DynamicRHI()->RHIGetResource(InResource->GetRaw().Texture)->QueryInterface(InResourceRaw.GetInitReference());
			if (FAILED(Result))
			{
				return FAVResult(EAVResult::ErrorMapping, TEXT("Raw resource is not a 2D texture"), TEXT("D3D11"), Result);
			}
			
			OutResource = MakeShared<FVideoResourceD3D11>(
				InResource->GetDevice(),
				InResourceRaw,
				InResource->GetLayout());

			return OutResource->Validate();
		}

		return FAVResult(EAVResult::ErrorMapping, TEXT("No D3D11 context found"), TEXT("RHI"));
	}

	return FAVResult(EAVResult::ErrorMapping, TEXT("Input resource is not valid"), TEXT("RHI"));
}

template <>
DLLEXPORT FAVResult FAVExtension::TransformResource(TSharedPtr<FVideoResourceD3D12>& OutResource, TSharedPtr<FVideoResourceRHI> const& InResource)
{
	if (InResource.IsValid())
	{
		if (InResource->GetDevice()->HasContext<FVideoContextD3D12>())
		{
			OutResource = MakeShareable( new FVideoResourceD3D12(
				InResource->GetDevice(),
				{	
					GetID3D12DynamicRHI()->RHIGetResource(InResource->GetRaw().Texture),
					nullptr,
					nullptr, // GetID3D12DynamicRHI()->RHIGetHeap(InResource->GetRaw().Texture),
					nullptr, 
					InResource->GetRaw().Fence, 
					nullptr, 
					0
				},
				InResource->GetLayout(),
				InResource->GetDescriptor()));

			return OutResource->Validate();
		}

		return FAVResult(EAVResult::ErrorMapping, TEXT("No D3D12 context found"), TEXT("RHI"));
	}

	return FAVResult(EAVResult::ErrorMapping, TEXT("Input resource is not valid"), TEXT("RHI"));
}

#endif

#if AVCODECS_USE_VULKAN
template <>
DLLEXPORT FAVResult FAVExtension::TransformResource(TSharedPtr<FVideoResourceVulkan>& OutResource, TSharedPtr<FVideoResourceRHI> const& InResource)
{
	if (InResource.IsValid())
	{
		if (InResource->GetDevice()->HasContext<FVideoContextVulkan>())
		{
			OutResource = MakeShared<FVideoResourceVulkan>(
				InResource->GetDevice(),
				GetIVulkanDynamicRHI()->RHIGetAllocationInfo(InResource->GetRaw().Texture.GetReference()).Handle,
				InResource->GetLayout(),
				InResource->GetDescriptor());

			return OutResource->Validate();
		}

		return FAVResult(EAVResult::ErrorMapping, TEXT("No Vulkan context found"), TEXT("RHI"));
	}

	return FAVResult(EAVResult::ErrorMapping, TEXT("Input resource is not valid"), TEXT("RHI"));
}
#endif

#if AVCODECS_USE_METAL
template <>
DLLEXPORT FAVResult FAVExtension::TransformResource(TSharedPtr<FVideoResourceMetal>& OutResource, TSharedPtr<FVideoResourceRHI> const& InResource)
{
	if (InResource.IsValid())
	{
		if (InResource->GetDevice()->HasContext<FVideoContextMetal>())
		{
			FTextureRHIRef InResourceTexture = InResource->GetRaw().Texture;
            if(bool(InResourceTexture->GetDesc().Flags & TexCreate_CPUReadback))
            {
                OutResource = MakeShared<FVideoResourceMetal>(InResource->GetDevice(),
															  static_cast<MTL::Texture*>(InResource->GetRaw().Texture->GetNativeResource()),
                                                              InResource->GetLayout());
			
			    return OutResource->Validate();
            }
            else
            {
                static bool bLogWarning = true;
				if(bLogWarning)
				{
					bLogWarning = false;
					FAVResult::Log(EAVResult::Warning, TEXT("Unable to transform video resource! Metal RHI requires FVideoResourceRHI textures be created with the ETextureCreateFlags::CPUReadback flag"), TEXT("RHI"));
				}

				return EAVResult::Error;
            }
		}

		return FAVResult(EAVResult::ErrorMapping, TEXT("No Metal context found"), TEXT("RHI"));
	}

	return FAVResult(EAVResult::ErrorMapping, TEXT("Input resource is not valid"), TEXT("RHI"));
}
#endif
