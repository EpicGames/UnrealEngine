// Copyright Epic Games, Inc. All Rights Reserved.

#include "Video/SimpleVideo.h"

#include "Engine/TextureRenderTarget2D.h"

#include "Video/Decoders/Configs/VideoDecoderConfigH264.h"
#include "Video/Decoders/Configs/VideoDecoderConfigH265.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH264.h"
#include "Video/Encoders/Configs/VideoEncoderConfigH265.h"

#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimpleVideo)

ESimpleVideoCodec USimpleVideoHelper::GuessCodec(TSharedRef<FAVInstance> const& Instance)
{
	if (Instance->Has<FVideoEncoderConfigH264>() || Instance->Has<FVideoDecoderConfigH264>())
	{
		return ESimpleVideoCodec::H264;
	}
	else if (Instance->Has<FVideoEncoderConfigH265>() || Instance->Has<FVideoDecoderConfigH265>())
	{
		return ESimpleVideoCodec::H265;
	}

	return ESimpleVideoCodec::H264;
}

void USimpleVideoHelper::ShareRenderTarget2D(UTextureRenderTarget2D* RenderTarget)
{
	if (RenderTarget != nullptr && !RenderTarget->bGPUSharedFlag)
	{
		RenderTarget->bGPUSharedFlag = true;
		RenderTarget->UpdateResource();
		RenderTarget->UpdateResourceImmediate();

		FlushRenderingCommands();
	}
}
