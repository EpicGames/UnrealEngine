// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SampleBuffer.h"

#include "Audio/SimpleAudio.h"
#include "Audio/AudioEncoder.h"

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "HAL/Runnable.h"

#include "SimpleAudioEncoder.generated.h"

#define UE_API AVCODECSCORERHI_API

USTRUCT(BlueprintType)
struct FSimpleAudioEncoderConfig
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Audio")
	int32 Bitrate;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Audio")
	int32 Samplerate;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Audio")
	int32 NumChannels;
	
	FSimpleAudioEncoderConfig(ESimpleAVPreset Preset = ESimpleAVPreset::Default)
		: FSimpleAudioEncoderConfig(FAudioEncoderConfig(USimpleAudioHelper::ConvertPreset(Preset)))
	{
	}

	FSimpleAudioEncoderConfig(FAudioEncoderConfig const& From)
		: Bitrate(From.Bitrate)
		, Samplerate(From.Samplerate)
		, NumChannels(From.NumChannels)
	{
	}
	
	operator FAudioEncoderConfig() const
    {
		FAudioEncoderConfig Result;
		Result.Bitrate = Bitrate;
		Result.Samplerate = Samplerate;
		Result.NumChannels = NumChannels;

		return Result;
    }
};

UCLASS(MinimalAPI, Blueprintable)
class USimpleAudioEncoder : public UObject, public FRunnable
{
	GENERATED_BODY()

private:
	TSharedPtr<TAudioEncoder<class FAudioResourceCPU>> Child;

	struct FAsyncFrame
	{
		TSharedPtr<FAudioResourceCPU> Resource;
		uint32 Timestamp;

		FAsyncFrame() = default;
		FAsyncFrame(TSharedPtr<FAudioResourceCPU> const& Resource, uint32 Timestamp)
			: Resource(Resource)
			, Timestamp(Timestamp)
		{
		}
	};

	FRunnableThread* AsyncThread = nullptr;
	TQueue<FAsyncFrame> AsyncQueue;
	TQueue<TSharedPtr<FAudioResourceCPU>> AsyncPool;

	UE_API virtual uint32 Run() override;
	UE_API virtual void Exit() override;

public:
	UFUNCTION(BlueprintCallable, Category = "Audio")
	UE_API bool IsAsync() const;

	UFUNCTION(BlueprintCallable, Category = "Audio")
	UE_API bool IsOpen() const;

	UFUNCTION(BlueprintCallable, Category = "Audio")
	UE_API UPARAM(DisplayName = "Was Success") bool Open(ESimpleAudioCodec Codec, FSimpleAudioEncoderConfig Config, bool bAsynchronous);

	UFUNCTION(BlueprintCallable, Category = "Audio")
	UE_API void Close();
	
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (DisplayName = "Send Frame (PCM16)"))
	UE_API UPARAM(DisplayName = "Was Success") bool SendFrameFloat(TArray<float> const& Resource, double Timestamp, int32 NumSamples, float SampleDuration);

	UE_API bool SendFrame(Audio::TSampleBuffer<float> const& Resource, double Timestamp);
	UE_API bool SendFrame(float const* ResourceData, double Timestamp, int32 NumSamples, float SampleDuration);

	UFUNCTION(BlueprintCallable, Category = "Audio")
	UE_API UPARAM(DisplayName = "Was Success") bool ReceivePacket(FSimpleAudioPacket& OutPacket);

	UFUNCTION(BlueprintCallable, Category = "Audio")
	UE_API void ReceivePackets(TArray<FSimpleAudioPacket>& OutPackets);

	// Configuration
public:
	UFUNCTION(BlueprintCallable, Category = "Audio")
	UE_API ESimpleAudioCodec GetCodec() const;

	UFUNCTION(BlueprintCallable, Category = "Audio")
	UE_API FSimpleAudioEncoderConfig GetConfig() const;
	
	UFUNCTION(BlueprintCallable, Category = "Audio")
	UE_API void SetConfig(FSimpleAudioEncoderConfig NewConfig);
};

#undef UE_API
