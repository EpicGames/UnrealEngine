// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVConfig.h"
#include "Audio/AudioPacket.h"

#include "SimpleAV.h"

#include "SimpleAudio.generated.h"

#define UE_API AVCODECSCORERHI_API

UENUM(BlueprintType)
enum class ESimpleAudioCodec : uint8
{
	AAC = 0,
};

USTRUCT(BlueprintType)
struct FSimpleAudioPacket
{
	GENERATED_BODY()

public:
	FAudioPacket RawPacket;
};

UCLASS(MinimalAPI, Abstract)
class USimpleAudioHelper : public USimpleAVHelper
{
	GENERATED_BODY()

public:
	static UE_API ESimpleAudioCodec GuessCodec(TSharedRef<FAVInstance> const& Instance);
};

#undef UE_API
