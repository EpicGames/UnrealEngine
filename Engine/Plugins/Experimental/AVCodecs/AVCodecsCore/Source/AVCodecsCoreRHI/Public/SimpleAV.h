// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVConfig.h"

#include "SimpleAV.generated.h"

#define UE_API AVCODECSCORERHI_API

UENUM(BlueprintType)
enum class ESimpleAVPreset : uint8
{
	Default,
	
	UltraLowQuality,
	LowQuality,
	HighQuality,
	Lossless,
};

UCLASS(MinimalAPI, Abstract)
class USimpleAVHelper : public UObject
{
	GENERATED_BODY()

public:
	static UE_API EAVPreset ConvertPreset(ESimpleAVPreset From);
};

#undef UE_API
