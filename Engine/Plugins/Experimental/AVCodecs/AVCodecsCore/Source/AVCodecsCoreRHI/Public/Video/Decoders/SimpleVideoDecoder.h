// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/SimpleVideo.h"
#include "Video/VideoDecoder.h"

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "HAL/Runnable.h"

#include "SimpleVideoDecoder.generated.h"

#define UE_API AVCODECSCORERHI_API

UCLASS(MinimalAPI, Blueprintable)
class USimpleVideoDecoder : public UObject, public FRunnable
{
	GENERATED_BODY()

private:
	TSharedPtr<TVideoDecoder<class FVideoResourceRHI>> Child;

	FRunnableThread* AsyncThread = nullptr;
	TQueue<FSimpleVideoPacket> AsyncQueue;

	UE_API virtual uint32 Run() override;
	UE_API virtual void Exit() override;

public:
	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API bool IsAsync() const;

	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API bool IsOpen() const;

	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API UPARAM(DisplayName = "Was Success") bool Open(ESimpleVideoCodec Codec, bool bAsynchronous);

	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API void Close();

	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API UPARAM(DisplayName = "Was Success") bool SendPacket(FSimpleVideoPacket const& Packet);

	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API UPARAM(DisplayName = "Was Success") bool ReceiveFrame(UTextureRenderTarget2D* Resource);

	// Configuration
public:
	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API ESimpleVideoCodec GetCodec() const;
};

#undef UE_API
