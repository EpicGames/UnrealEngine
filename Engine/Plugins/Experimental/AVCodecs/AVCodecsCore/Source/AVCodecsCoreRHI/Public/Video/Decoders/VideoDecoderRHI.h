// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoDecoder.h"
#include "Video/Resources/VideoResourceRHI.h"

#if AVCODECS_USE_D3D
#include "Video/Resources/D3D/VideoResourceD3D.h"
#include "Video/Resources/Vulkan/VideoResourceVulkan.h"
#endif
#if AVCODECS_USE_VULKAN
#include "Video/Resources/Vulkan/VideoResourceVulkan.h"
#endif
#if AVCODECS_USE_METAL
#include "Video/Resources/Metal/VideoResourceMetal.h"
#endif

template <typename TConfig>
class TVideoDecoderRHI : public TVideoDecoder<FVideoResourceRHI, TConfig>
{
public:
	template <typename TChildConfig>
	static FVideoDecoder::TFactory<FVideoResourceRHI, TConfig>& Register()
	{
		return FVideoDecoder::Register<TVideoDecoderRHI<TConfig>, FVideoResourceRHI, TConfig>();
	}
	
	TSharedPtr<TVideoDecoder<FVideoResourceRHI, TConfig>> Child;

	TVideoDecoderRHI() = default;

	virtual bool IsOpen() const override
	{
		return Child.IsValid() && Child->IsOpen();
	}

	virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override
	{
		if (this->IsOpen())
		{
			return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder already open"), TEXT("RHI"));
		}

		FAVResult const Result = TVideoDecoder<FVideoResourceRHI, TConfig>::Open(NewDevice, NewInstance);
		if (Result.IsNotSuccess())
		{
			return Result;
		}

		switch (GDynamicRHI->GetInterfaceType())
		{
#if AVCODECS_USE_VULKAN
		case ERHIInterfaceType::Vulkan:
			Child = FVideoDecoder::Wrap<FVideoResourceRHI, TConfig>(FVideoDecoder::Create<FVideoResourceVulkan, TConfig>(NewDevice, NewInstance));
		
			break;
#endif
#if AVCODECS_USE_D3D
		case ERHIInterfaceType::D3D11:
			Child = FVideoDecoder::Wrap<FVideoResourceRHI, TConfig>(FVideoDecoder::Create<FVideoResourceD3D11, TConfig>(NewDevice, NewInstance));
		
			break;
		case ERHIInterfaceType::D3D12:
			Child = FVideoDecoder::Wrap<FVideoResourceRHI, TConfig>(FVideoDecoder::Create<FVideoResourceD3D12, TConfig>(NewDevice, NewInstance));
		
			break;
#endif
#if AVCODECS_USE_METAL
		case ERHIInterfaceType::Metal:
			Child = FVideoDecoder::Wrap<FVideoResourceRHI, TConfig>(FVideoDecoder::Create<FVideoResourceMetal, TConfig>(NewDevice, NewInstance));

			break;
#endif
		default:
			return FAVResult(EAVResult::ErrorCreating, FString::Printf(TEXT("RHI type %d not supported"), EnumToUnderlyingType(GDynamicRHI->GetInterfaceType())), TEXT("RHI"));
		}

		if (!IsOpen())
		{
			return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create RHI child platform decoder"), TEXT("RHI"));
		}

		return EAVResult::Success;
	}
	
	virtual void Close() override
	{
		if (this->IsOpen())
		{
			Child.Reset();
		}
	}

	virtual FAVResult ApplyConfig() override
	{
		if (!this->IsOpen())
		{
			return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("RHI"));
		}

		FAVResult const Result = TVideoDecoder<FVideoResourceRHI, TConfig>::ApplyConfig();
		if (Result.IsNotSuccess())
		{
			return Result;
		}

		Child->SetPendingConfig(this->GetAppliedConfig());

		return EAVResult::Success;
	}

	virtual FAVResult SendPacket(FVideoPacket const& Packet) override
	{
		if (!this->IsOpen())
		{
			return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("RHI"));
		}

		FAVResult const Result = this->ApplyConfig();
		if (Result.IsNotSuccess())
		{
			return Result;
		}

		return Child->SendPacket(Packet);
	}

	virtual FAVResult ReceiveFrame(TResolvableVideoResource<FVideoResourceRHI>& InOutResource) override
	{
		if (!this->IsOpen())
		{
			return FAVResult(EAVResult::ErrorInvalidState, TEXT("Decoder not open"), TEXT("RHI"));
		}

		return Child->ReceiveFrame(InOutResource);
	}
};
