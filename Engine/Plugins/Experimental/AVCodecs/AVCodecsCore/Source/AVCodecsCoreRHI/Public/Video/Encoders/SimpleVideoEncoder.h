// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/SimpleVideo.h"
#include "Video/VideoEncoder.h"

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "HAL/Runnable.h"
#include "HAL/Platform.h"
#include "RHIFwd.h"

#include "SimpleVideoEncoder.generated.h"

#define UE_API AVCODECSCORERHI_API

USTRUCT(BlueprintType)
struct FSimpleVideoEncoderConfig
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Video")
	int32 Width;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Video")
	int32 Height;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Video")
	int32 TargetFramerate;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Video")
	int32 TargetBitrate;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Video")
	int32 MaxBitrate;
	
	FSimpleVideoEncoderConfig(ESimpleAVPreset Preset = ESimpleAVPreset::Default)
		: FSimpleVideoEncoderConfig(FVideoEncoderConfig(USimpleVideoHelper::ConvertPreset(Preset)))
	{
	}

	FSimpleVideoEncoderConfig(FVideoEncoderConfig const& From)
		: Width(From.Width)
		, Height(From.Height)
		, TargetFramerate(From.TargetFramerate)
		, TargetBitrate(From.TargetBitrate)
		, MaxBitrate(From.MaxBitrate)
	{
	}
	
	operator FVideoEncoderConfig() const
    {
		FVideoEncoderConfig Result;
		Result.Width = Width;
		Result.Height = Height;
		Result.TargetFramerate = TargetFramerate;
		Result.TargetBitrate = TargetBitrate;
		Result.MaxBitrate = MaxBitrate;

		return Result;
    }
};

UCLASS(MinimalAPI, Blueprintable)
class USimpleVideoEncoder : public UObject, public FRunnable
{
	GENERATED_BODY()

private:
	TSharedPtr<TVideoEncoder<class FVideoResourceRHI>> Child;

	struct FAsyncFrame
	{
		TSharedPtr<FVideoResourceRHI> Resource;
		uint32 Timestamp;
		bool bForceKeyframe;

		FAsyncFrame() = default;
		FAsyncFrame(TSharedPtr<FVideoResourceRHI> const& Resource, uint32 Timestamp, bool bForceKeyframe)
			: Resource(Resource)
			, Timestamp(Timestamp)
			, bForceKeyframe(bForceKeyframe)
		{
		}
	};

	FRunnableThread* AsyncThread = nullptr;
	TQueue<FAsyncFrame> AsyncQueue;
	TQueue<TSharedPtr<FVideoResourceRHI>> AsyncPool;

	UE_API virtual uint32 Run() override;
	UE_API virtual void Exit() override;

public:
	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API bool IsAsync() const;

	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API bool IsOpen() const;

	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API UPARAM(DisplayName = "Was Success") bool Open(ESimpleVideoCodec Codec, FSimpleVideoEncoderConfig Config, bool bAsynchronous);

	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API void Close();

	UFUNCTION(BlueprintCallable, Category = "Video", meta = (DisplayName = "Send Frame (Render Target)"))
	UE_API UPARAM(DisplayName = "Was Success") bool SendFrameRenderTarget(UTextureRenderTarget2D* Resource, double Timestamp, bool bForceKeyframe = false);
	
	UFUNCTION(BlueprintCallable, Category = "Video", meta = (DisplayName = "Send Frame (Texture)"))
	UE_API UPARAM(DisplayName = "Was Success") bool SendFrameTexture(UTexture2D* Resource, double Timestamp, bool bForceKeyframe = false);

	UE_API bool SendFrame(FTextureRHIRef const& Resource, double Timestamp, bool bForceKeyframe = false);

	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API UPARAM(DisplayName = "Was Success") bool ReceivePacket(FSimpleVideoPacket& OutPacket);

	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API void ReceivePackets(TArray<FSimpleVideoPacket>& OutPackets);

	// Configuration
public:
	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API ESimpleVideoCodec GetCodec() const;

	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API FSimpleVideoEncoderConfig GetConfig() const;
	
	UFUNCTION(BlueprintCallable, Category = "Video")
	UE_API void SetConfig(FSimpleVideoEncoderConfig NewConfig);
};

#undef UE_API
