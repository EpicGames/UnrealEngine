// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Video/VideoEncoder.h"
#include "Video/Resources/VideoResourceRHI.h"

#if AVCODECS_USE_D3D
#include "Video/Resources/D3D/VideoResourceD3D.h"
#endif
#if AVCODECS_USE_VULKAN
#include "Video/Resources/Vulkan/VideoResourceVulkan.h"
#endif
#if AVCODECS_USE_METAL
#include "Video/Resources/Metal/VideoResourceMetal.h"
#endif

template <typename TConfig>
class TVideoEncoderRHI : public TVideoEncoder<FVideoResourceRHI, TConfig>
{
public:
	template <typename TChildConfig>
	static FVideoEncoder::TFactory<FVideoResourceRHI, TConfig>& Register()
	{
		return FVideoEncoder::Register<TVideoEncoderRHI<TConfig>, FVideoResourceRHI, TConfig>();
	}
	
	TSharedPtr<TVideoEncoder<FVideoResourceRHI, TConfig>> Child;

	TVideoEncoderRHI() = default;

	virtual bool IsOpen() const override
	{
		return Child.IsValid() && Child->IsOpen();
	}

	virtual FAVResult Open(TSharedRef<FAVDevice> const& NewDevice, TSharedRef<FAVInstance> const& NewInstance) override
	{
		if (IsOpen())
		{
			return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder already open"), TEXT("RHI"));
		}

		FAVResult const Result = TVideoEncoder<FVideoResourceRHI, TConfig>::Open(NewDevice, NewInstance);
		if (Result.IsNotSuccess())
		{
			return Result;
		}

		switch (GDynamicRHI->GetInterfaceType())
		{
#if AVCODECS_USE_VULKAN
		case ERHIInterfaceType::Vulkan:
			Child = FVideoEncoder::Wrap<FVideoResourceRHI, TConfig>(FVideoEncoder::Create<FVideoResourceVulkan, TConfig>(NewDevice, NewInstance));
		
			break;
#endif
#if AVCODECS_USE_D3D
		case ERHIInterfaceType::D3D11:
			Child = FVideoEncoder::Wrap<FVideoResourceRHI, TConfig>(FVideoEncoder::Create<FVideoResourceD3D11, TConfig>(NewDevice, NewInstance));
		
			break;
		case ERHIInterfaceType::D3D12:
			Child = FVideoEncoder::Wrap<FVideoResourceRHI, TConfig>(FVideoEncoder::Create<FVideoResourceD3D12, TConfig>(NewDevice, NewInstance));
		
			break;
#endif
#if AVCODECS_USE_METAL
        case ERHIInterfaceType::Metal:
			Child = FVideoEncoder::Wrap<FVideoResourceRHI, TConfig>(FVideoEncoder::Create<FVideoResourceMetal, TConfig>(NewDevice, NewInstance));
		
			break;
#endif
		default:
			return FAVResult(EAVResult::ErrorCreating, FString::Printf(TEXT("RHI type %d not supported"), EnumToUnderlyingType(GDynamicRHI->GetInterfaceType())), TEXT("RHI"));
		}

		if (!IsOpen())
		{
			return FAVResult(EAVResult::ErrorCreating, TEXT("Failed to create RHI child platform encoder"), TEXT("RHI"));
		}

		return EAVResult::Success;
	}
	
	virtual void Close() override
	{
		if (this->IsOpen())
		{
			Child.Reset();
		}
	}

	virtual FAVResult ApplyConfig() override
	{
		if (!this->IsOpen())
		{
			return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("RHI"));
		}

		FAVResult const Result = TVideoEncoder<FVideoResourceRHI, TConfig>::ApplyConfig();
		if (Result.IsNotSuccess())
		{
			return Result;
		}

		Child->SetPendingConfig(this->GetAppliedConfig());

		return EAVResult::Success;
	}

	virtual FAVResult SendFrame(TSharedPtr<FVideoResourceRHI> const& Resource, uint32 Timestamp, bool bForceKeyframe) override
	{
		if (!this->IsOpen())
		{
			return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("RHI"));
		}

		FAVResult const Result = this->ApplyConfig();
		if (Result.IsNotSuccess())
		{
			return Result;
		}

		return Child->SendFrame(Resource, Timestamp, bForceKeyframe);
	}

	virtual FAVResult ReceivePacket(FVideoPacket& OutPacket) override
	{
		if (!this->IsOpen())
		{
			return FAVResult(EAVResult::ErrorInvalidState, TEXT("Encoder not open"), TEXT("RHI"));
		}

		return Child->ReceivePacket(OutPacket);
	}
};
