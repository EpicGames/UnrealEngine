// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVContext.h"
#include "AVExtension.h"
#include "Video/VideoResource.h"
#include "RHI.h"

#if AVCODECS_USE_D3D
THIRD_PARTY_INCLUDES_START
#include "Windows/WindowsHWrapper.h"
#include "d3d12.h"
THIRD_PARTY_INCLUDES_END
#endif

class FVideoContextRHI : public FAVContext
{
};

class FVideoResourceRHI : public TVideoResource<FVideoContextRHI>
{
public:
	struct FRawData
	{
		FTextureRHIRef Texture;
#if AVCODECS_USE_D3D
		TRefCountPtr<ID3D12Fence> Fence; // TODO replace with an RHI fence and convert in a transform function
#else 
		FGPUFenceRHIRef Fence;
#endif
		uint64 FenceValue;
	};

private:
	FRawData Raw;

public:
	static AVCODECSCORERHI_API FAVLayout GetLayoutFrom(TSharedRef<FAVDevice> const& Device, FTextureRHIRef const& Raw);
	static AVCODECSCORERHI_API FVideoDescriptor GetDescriptorFrom(TSharedRef<FAVDevice> const& Device, FTextureRHIRef const& Raw);

	static AVCODECSCORERHI_API TSharedPtr<FVideoResourceRHI> Create(TSharedPtr<FAVDevice> const& Device, FVideoDescriptor const& Descriptor, ETextureCreateFlags AdditionalFlags = ETextureCreateFlags::None, bool bIsSRGB = true);


	inline FRawData const& GetRaw() const { return Raw; }

	AVCODECSCORERHI_API FVideoResourceRHI(TSharedRef<FAVDevice> const& Device, FRawData const& Raw, FVideoDescriptor const& OverrideDescriptor);
	AVCODECSCORERHI_API FVideoResourceRHI(TSharedRef<FAVDevice> const& Device, FRawData const& Raw);
	virtual ~FVideoResourceRHI() override = default;

	AVCODECSCORERHI_API virtual FAVResult Validate() const override;

	AVCODECSCORERHI_API virtual void Lock() override;
	AVCODECSCORERHI_API virtual FScopeLock LockScope() override;

	AVCODECSCORERHI_API void CopyFrom(TArrayView64<uint8> const& From);
	AVCODECSCORERHI_API void CopyFrom(TArrayView<uint8> const& From);
	AVCODECSCORERHI_API void CopyFrom(TArray64<uint8> const& From);
	AVCODECSCORERHI_API void CopyFrom(TArray<uint8> const& From);
	AVCODECSCORERHI_API void CopyFrom(FTextureRHIRef const& From);

	AVCODECSCORERHI_API TSharedPtr<FVideoResourceRHI> TransformResource(FVideoDescriptor const& OutDescriptor);
	AVCODECSCORERHI_API void TransformResourceTo(FRHICommandListImmediate& RHICmdList, FTextureRHIRef Target, FVideoDescriptor const& OutDescriptor);
};

class FResolvableVideoResourceRHI : public TResolvableVideoResource<FVideoResourceRHI>
{
public:
	ETextureCreateFlags AdditionalFlags = ETextureCreateFlags::None;
	
protected:
	AVCODECSCORERHI_API virtual TSharedPtr<FVideoResourceRHI> TryResolve(TSharedPtr<FAVDevice> const& Device, FVideoDescriptor const& Descriptor) override;
};

/*
* Handles pixel format or colorspace transformations
*/
template <>
FAVResult FAVExtension::TransformResource(TSharedPtr<FVideoResourceRHI>& OutResource, TSharedPtr<class FVideoResourceRHI> const& InResource);

template <>
FAVResult FAVExtension::TransformResource(TSharedPtr<FVideoResourceRHI>& OutResource, TSharedPtr<class FVideoResourceCPU> const& InResource);

#if AVCODECS_USE_D3D

template <>
FAVResult FAVExtension::TransformResource(TSharedPtr<class FVideoResourceD3D11>& OutResource, TSharedPtr<FVideoResourceRHI> const& InResource);

template <>
FAVResult FAVExtension::TransformResource(TSharedPtr<class FVideoResourceD3D12>& OutResource, TSharedPtr<FVideoResourceRHI> const& InResource);

#endif

#if AVCODECS_USE_VULKAN

template <>
FAVResult FAVExtension::TransformResource(TSharedPtr<class FVideoResourceVulkan>& OutResource, TSharedPtr<FVideoResourceRHI> const& InResource);

#endif

#if AVCODECS_USE_METAL

template <>
FAVResult FAVExtension::TransformResource(TSharedPtr<class FVideoResourceMetal>& OutResource, TSharedPtr<FVideoResourceRHI> const& InResource);

#endif

DECLARE_TYPEID(FVideoContextRHI, AVCODECSCORERHI_API);
DECLARE_TYPEID(FVideoResourceRHI, AVCODECSCORERHI_API);
