// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AVConfig.h"
#include "Video/VideoPacket.h"

#include "SimpleAV.h"

#include "SimpleVideo.generated.h"

#define UE_API AVCODECSCORERHI_API

UENUM(BlueprintType)
enum class ESimpleVideoCodec : uint8
{
	H264 = 0,
	H265 = 1,
};

USTRUCT(BlueprintType)
struct FSimpleVideoPacket
{
	GENERATED_BODY()

public:
	FVideoPacket RawPacket;
};

UCLASS(MinimalAPI, Abstract)
class USimpleVideoHelper : public USimpleAVHelper
{
	GENERATED_BODY()

public:
	static UE_API ESimpleVideoCodec GuessCodec(TSharedRef<FAVInstance> const& Instance);

	UFUNCTION(BlueprintCallable, Category = "Video", meta = (WorldContext = "WorldContext"))
	static UE_API void ShareRenderTarget2D(class UTextureRenderTarget2D* RenderTarget);
};

#undef UE_API
